// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#if PLATFORM_MICROSOFT
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Templates/RefCounting.h"

#ifndef RHICORE_PLATFORM_DXGI_H
	#error "Platform needs to define RHICORE_PLATFORM_DXGI_H"
#endif

#include "PixelFormat.h"
#include "RHIDefinitions.h"
#include "Misc/AssertionMacros.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include RHICORE_PLATFORM_DXGI_H
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

struct FRHIMemoryStats;

namespace UE::DXGIUtilities
{
	RHICORE_API const TCHAR* GetFormatString(DXGI_FORMAT Format);
	RHICORE_API HRESULT GetD3DMemoryStats(IDXGIAdapter* Adapter, FRHIMemoryStats& OutStats);

	inline DXGI_FORMAT FindSharedResourceFormat(DXGI_FORMAT InFormat, bool bSRGB)
	{
		if (bSRGB)
		{
			switch (InFormat)
			{
			case DXGI_FORMAT_B8G8R8X8_TYPELESS: return DXGI_FORMAT_B8G8R8X8_UNORM_SRGB;
			case DXGI_FORMAT_B8G8R8A8_TYPELESS: return DXGI_FORMAT_B8G8R8A8_UNORM_SRGB;
			case DXGI_FORMAT_R8G8B8A8_TYPELESS: return DXGI_FORMAT_R8G8B8A8_UNORM_SRGB;
			case DXGI_FORMAT_BC1_TYPELESS:      return DXGI_FORMAT_BC1_UNORM_SRGB;
			case DXGI_FORMAT_BC2_TYPELESS:      return DXGI_FORMAT_BC2_UNORM_SRGB;
			case DXGI_FORMAT_BC3_TYPELESS:      return DXGI_FORMAT_BC3_UNORM_SRGB;
			case DXGI_FORMAT_BC7_TYPELESS:      return DXGI_FORMAT_BC7_UNORM_SRGB;
			};
		}
		else
		{
			switch (InFormat)
			{
			case DXGI_FORMAT_B8G8R8X8_TYPELESS: return DXGI_FORMAT_B8G8R8X8_UNORM;
			case DXGI_FORMAT_B8G8R8A8_TYPELESS: return DXGI_FORMAT_B8G8R8A8_UNORM;
			case DXGI_FORMAT_R8G8B8A8_TYPELESS: return DXGI_FORMAT_R8G8B8A8_UNORM;
			case DXGI_FORMAT_BC1_TYPELESS:      return DXGI_FORMAT_BC1_UNORM;
			case DXGI_FORMAT_BC2_TYPELESS:      return DXGI_FORMAT_BC2_UNORM;
			case DXGI_FORMAT_BC3_TYPELESS:      return DXGI_FORMAT_BC3_UNORM;
			case DXGI_FORMAT_BC7_TYPELESS:      return DXGI_FORMAT_BC7_UNORM;
			};
		}

		switch (InFormat)
		{
		case DXGI_FORMAT_R32G32B32A32_TYPELESS: return DXGI_FORMAT_R32G32B32A32_UINT;
		case DXGI_FORMAT_R32G32B32_TYPELESS:    return DXGI_FORMAT_R32G32B32_UINT;
		case DXGI_FORMAT_R16G16B16A16_TYPELESS: return DXGI_FORMAT_R16G16B16A16_UNORM;
		case DXGI_FORMAT_R32G32_TYPELESS:       return DXGI_FORMAT_R32G32_UINT;
		case DXGI_FORMAT_R10G10B10A2_TYPELESS:  return DXGI_FORMAT_R10G10B10A2_UNORM;
		case DXGI_FORMAT_R16G16_TYPELESS:       return DXGI_FORMAT_R16G16_UNORM;
		case DXGI_FORMAT_R8G8_TYPELESS:         return DXGI_FORMAT_R8G8_UNORM;
		case DXGI_FORMAT_R8_TYPELESS:           return DXGI_FORMAT_R8_UNORM;

		case DXGI_FORMAT_BC4_TYPELESS:          return DXGI_FORMAT_BC4_UNORM;
		case DXGI_FORMAT_BC5_TYPELESS:          return DXGI_FORMAT_BC5_UNORM;
		case DXGI_FORMAT_R24G8_TYPELESS:        return DXGI_FORMAT_R24_UNORM_X8_TYPELESS;
		case DXGI_FORMAT_R32_TYPELESS:          return DXGI_FORMAT_R32_FLOAT;
		case DXGI_FORMAT_R16_TYPELESS:          return DXGI_FORMAT_R16_UNORM;

		case DXGI_FORMAT_R32G8X24_TYPELESS:     return DXGI_FORMAT_R32_FLOAT_X8X24_TYPELESS;
		}

		return InFormat;
	}

	/** Find an appropriate DXGI format for the input format and SRGB setting. */
	inline DXGI_FORMAT FindShaderResourceFormat(DXGI_FORMAT InFormat, bool bSRGB)
	{
		if (bSRGB)
		{
			switch (InFormat)
			{
			case DXGI_FORMAT_B8G8R8A8_TYPELESS:    return DXGI_FORMAT_B8G8R8A8_UNORM_SRGB;
			case DXGI_FORMAT_R8G8B8A8_TYPELESS:    return DXGI_FORMAT_R8G8B8A8_UNORM_SRGB;
			case DXGI_FORMAT_BC1_TYPELESS:         return DXGI_FORMAT_BC1_UNORM_SRGB;
			case DXGI_FORMAT_BC2_TYPELESS:         return DXGI_FORMAT_BC2_UNORM_SRGB;
			case DXGI_FORMAT_BC3_TYPELESS:         return DXGI_FORMAT_BC3_UNORM_SRGB;
			case DXGI_FORMAT_BC7_TYPELESS:         return DXGI_FORMAT_BC7_UNORM_SRGB;
			};
		}
		else
		{
			switch (InFormat)
			{
			case DXGI_FORMAT_B8G8R8A8_TYPELESS: return DXGI_FORMAT_B8G8R8A8_UNORM;
			case DXGI_FORMAT_R8G8B8A8_TYPELESS: return DXGI_FORMAT_R8G8B8A8_UNORM;
			case DXGI_FORMAT_BC1_TYPELESS:      return DXGI_FORMAT_BC1_UNORM;
			case DXGI_FORMAT_BC2_TYPELESS:      return DXGI_FORMAT_BC2_UNORM;
			case DXGI_FORMAT_BC3_TYPELESS:      return DXGI_FORMAT_BC3_UNORM;
			case DXGI_FORMAT_BC7_TYPELESS:      return DXGI_FORMAT_BC7_UNORM;
			};
		}
		switch (InFormat)
		{
		case DXGI_FORMAT_R24G8_TYPELESS:    return DXGI_FORMAT_R24_UNORM_X8_TYPELESS;
		case DXGI_FORMAT_R32_TYPELESS:      return DXGI_FORMAT_R32_FLOAT;
		case DXGI_FORMAT_R16_TYPELESS:      return DXGI_FORMAT_R16_UNORM;
		case DXGI_FORMAT_R32G8X24_TYPELESS: return DXGI_FORMAT_R32_FLOAT_X8X24_TYPELESS;
		}
		return InFormat;
	}

	inline DXGI_FORMAT FindDepthStencilResourceFormat(DXGI_FORMAT InFormat)
	{
		switch (InFormat)
		{
		case DXGI_FORMAT_R32_FLOAT: return DXGI_FORMAT_R32_TYPELESS;
		case DXGI_FORMAT_R16_FLOAT: return DXGI_FORMAT_R16_TYPELESS;
		}

		return InFormat;
	}

	inline DXGI_FORMAT GetPlatformTextureResourceFormat(DXGI_FORMAT InFormat, ETextureCreateFlags InFlags)
	{
		// Find valid shared texture format
		if (EnumHasAnyFlags(InFlags, ETextureCreateFlags::Shared))
		{
			return FindSharedResourceFormat(InFormat, EnumHasAnyFlags(InFlags, ETextureCreateFlags::SRGB));
		}
		if (EnumHasAnyFlags(InFlags, ETextureCreateFlags::DepthStencilTargetable))
		{
			return FindDepthStencilResourceFormat(InFormat);
		}

		return InFormat;
	}

	inline DXGI_FORMAT GetPlatformTextureResourceFormat(EPixelFormat InPixelFormat, ETextureCreateFlags InFlags)
	{
		return GetPlatformTextureResourceFormat((DXGI_FORMAT)GPixelFormats[InPixelFormat].PlatformFormat, InFlags);
	}

	/** Find an appropriate DXGI format unordered access of the raw format. */
	inline DXGI_FORMAT FindUnorderedAccessFormat(DXGI_FORMAT InFormat)
	{
		switch (InFormat)
		{
		case DXGI_FORMAT_B8G8R8A8_TYPELESS: return DXGI_FORMAT_B8G8R8A8_UNORM;
		case DXGI_FORMAT_R8G8B8A8_TYPELESS: return DXGI_FORMAT_R8G8B8A8_UNORM;
		case DXGI_FORMAT_R32G8X24_TYPELESS: return DXGI_FORMAT_R32_FLOAT_X8X24_TYPELESS;
		}
		return InFormat;
	}

	/** Find the appropriate depth-stencil targetable DXGI format for the given format. */
	inline DXGI_FORMAT FindDepthStencilFormat(DXGI_FORMAT InFormat)
	{
		switch (InFormat)
		{
		case DXGI_FORMAT_R24G8_TYPELESS:    return DXGI_FORMAT_D24_UNORM_S8_UINT;
		case DXGI_FORMAT_R32G8X24_TYPELESS: return DXGI_FORMAT_D32_FLOAT_S8X24_UINT;
		case DXGI_FORMAT_R32_TYPELESS:      return DXGI_FORMAT_D32_FLOAT;
		case DXGI_FORMAT_R16_TYPELESS:      return DXGI_FORMAT_D16_UNORM;
		};
		return InFormat;
	}

	/** Find the appropriate depth-stencil typeless DXGI format for the given format. */
	inline DXGI_FORMAT FindDepthStencilParentDXGIFormat(DXGI_FORMAT InFormat)
	{
		switch (InFormat)
		{
		case DXGI_FORMAT_D24_UNORM_S8_UINT:			return DXGI_FORMAT_R24G8_TYPELESS;
		case DXGI_FORMAT_X24_TYPELESS_G8_UINT:		return DXGI_FORMAT_R24G8_TYPELESS;

		case DXGI_FORMAT_D32_FLOAT_S8X24_UINT:		return DXGI_FORMAT_R32G8X24_TYPELESS;
		case DXGI_FORMAT_X32_TYPELESS_G8X24_UINT:	return DXGI_FORMAT_R32G8X24_TYPELESS;

		case DXGI_FORMAT_D32_FLOAT:					return DXGI_FORMAT_R32_TYPELESS;
		case DXGI_FORMAT_D16_UNORM:					return DXGI_FORMAT_R16_TYPELESS;
		};
		return InFormat;
	}

	/**
	 * Returns whether the given format contains stencil information.
	 * Must be passed a format returned by FindDepthStencilFormat, so that typeless versions are converted to their corresponding depth stencil view format.
	 */
	inline bool HasStencilBits(DXGI_FORMAT InFormat)
	{
		switch (InFormat)
		{
		case DXGI_FORMAT_D24_UNORM_S8_UINT:
		case DXGI_FORMAT_D32_FLOAT_S8X24_UINT:
			return true;
		};
		return false;
	}

	inline bool IsBlockCompressedFormat(DXGI_FORMAT Format)
	{
		// Returns true if BC1, BC2, BC3, BC4, BC5, BC6, BC7
		return (Format >= DXGI_FORMAT_BC1_TYPELESS && Format <= DXGI_FORMAT_BC5_SNORM) ||
			(Format >= DXGI_FORMAT_BC6H_TYPELESS && Format <= DXGI_FORMAT_BC7_UNORM_SRGB);
	}

	inline uint8 GetPlaneSliceFromViewFormat(DXGI_FORMAT ResourceFormat, DXGI_FORMAT ViewFormat)
	{
		// Currently, the only planar resources used are depth-stencil formats
		switch (FindDepthStencilParentDXGIFormat(ResourceFormat))
		{
		case DXGI_FORMAT_R24G8_TYPELESS:
			switch (ViewFormat)
			{
			case DXGI_FORMAT_R24_UNORM_X8_TYPELESS: return 0;
			case DXGI_FORMAT_X24_TYPELESS_G8_UINT: return 1;
			}
			break;

		case DXGI_FORMAT_R32G8X24_TYPELESS:
			switch (ViewFormat)
			{
			case DXGI_FORMAT_R32_FLOAT_X8X24_TYPELESS: return 0;
			case DXGI_FORMAT_X32_TYPELESS_G8X24_UINT: return 1;
			}
			break;

		case DXGI_FORMAT_NV12:
			switch (ViewFormat)
			{
			case DXGI_FORMAT_R8_UNORM: return 0;
			case DXGI_FORMAT_R8G8_UNORM: return 1;
			}
			break;

		case DXGI_FORMAT_P010:
			switch (ViewFormat)
			{
			case DXGI_FORMAT_R16_UNORM: return 0;
			case DXGI_FORMAT_R16G16_UNORM: return 1;
			}
			break;
		}

		return 0;
	}

	inline uint8 GetPlaneCount(DXGI_FORMAT Format)
	{
		// Currently, the only planar resources used are depth-stencil formats
		switch (FindDepthStencilParentDXGIFormat(Format))
		{
		case DXGI_FORMAT_R24G8_TYPELESS:
		case DXGI_FORMAT_R32G8X24_TYPELESS:
			return 2;
		default:
			return 1;
		}
	}

	inline uint32 GetWidthAlignment(DXGI_FORMAT Format)
	{
		switch (Format)
		{
		case DXGI_FORMAT_R8G8_B8G8_UNORM: return 2;
		case DXGI_FORMAT_G8R8_G8B8_UNORM: return 2;
		case DXGI_FORMAT_NV12: return 2;
		case DXGI_FORMAT_P010: return 2;
		case DXGI_FORMAT_P016: return 2;
		case DXGI_FORMAT_420_OPAQUE: return 2;
		case DXGI_FORMAT_YUY2: return 2;
		case DXGI_FORMAT_Y210: return 2;
		case DXGI_FORMAT_Y216: return 2;
		case DXGI_FORMAT_BC1_TYPELESS: return 4;
		case DXGI_FORMAT_BC1_UNORM: return 4;
		case DXGI_FORMAT_BC1_UNORM_SRGB: return 4;
		case DXGI_FORMAT_BC2_TYPELESS: return 4;
		case DXGI_FORMAT_BC2_UNORM: return 4;
		case DXGI_FORMAT_BC2_UNORM_SRGB: return 4;
		case DXGI_FORMAT_BC3_TYPELESS: return 4;
		case DXGI_FORMAT_BC3_UNORM: return 4;
		case DXGI_FORMAT_BC3_UNORM_SRGB: return 4;
		case DXGI_FORMAT_BC4_TYPELESS: return 4;
		case DXGI_FORMAT_BC4_UNORM: return 4;
		case DXGI_FORMAT_BC4_SNORM: return 4;
		case DXGI_FORMAT_BC5_TYPELESS: return 4;
		case DXGI_FORMAT_BC5_UNORM: return 4;
		case DXGI_FORMAT_BC5_SNORM: return 4;
		case DXGI_FORMAT_BC6H_TYPELESS: return 4;
		case DXGI_FORMAT_BC6H_UF16: return 4;
		case DXGI_FORMAT_BC6H_SF16: return 4;
		case DXGI_FORMAT_BC7_TYPELESS: return 4;
		case DXGI_FORMAT_BC7_UNORM: return 4;
		case DXGI_FORMAT_BC7_UNORM_SRGB: return 4;
		case DXGI_FORMAT_NV11: return 4;
		case DXGI_FORMAT_R1_UNORM: return 8;
		default: return 1;
		}
	}

	inline uint32 GetHeightAlignment(DXGI_FORMAT Format)
	{
		switch (Format)
		{
		case DXGI_FORMAT_NV12: return 2;
		case DXGI_FORMAT_P010: return 2;
		case DXGI_FORMAT_P016: return 2;
		case DXGI_FORMAT_420_OPAQUE: return 2;
		case DXGI_FORMAT_BC1_TYPELESS: return 4;
		case DXGI_FORMAT_BC1_UNORM: return 4;
		case DXGI_FORMAT_BC1_UNORM_SRGB: return 4;
		case DXGI_FORMAT_BC2_TYPELESS: return 4;
		case DXGI_FORMAT_BC2_UNORM: return 4;
		case DXGI_FORMAT_BC2_UNORM_SRGB: return 4;
		case DXGI_FORMAT_BC3_TYPELESS: return 4;
		case DXGI_FORMAT_BC3_UNORM: return 4;
		case DXGI_FORMAT_BC3_UNORM_SRGB: return 4;
		case DXGI_FORMAT_BC4_TYPELESS: return 4;
		case DXGI_FORMAT_BC4_UNORM: return 4;
		case DXGI_FORMAT_BC4_SNORM: return 4;
		case DXGI_FORMAT_BC5_TYPELESS: return 4;
		case DXGI_FORMAT_BC5_UNORM: return 4;
		case DXGI_FORMAT_BC5_SNORM: return 4;
		case DXGI_FORMAT_BC6H_TYPELESS: return 4;
		case DXGI_FORMAT_BC6H_UF16: return 4;
		case DXGI_FORMAT_BC6H_SF16: return 4;
		case DXGI_FORMAT_BC7_TYPELESS: return 4;
		case DXGI_FORMAT_BC7_UNORM: return 4;
		case DXGI_FORMAT_BC7_UNORM_SRGB: return 4;
		default: return 1;
		}
	}

	inline uint32 GetFormatSizeInBits(DXGI_FORMAT Format)
	{
		switch (Format)
		{
		default: checkNoEntry(); [[fallthrough]];
		case DXGI_FORMAT_UNKNOWN:
			return 0;

		case DXGI_FORMAT_R32G32B32A32_TYPELESS:
		case DXGI_FORMAT_R32G32B32A32_FLOAT:
		case DXGI_FORMAT_R32G32B32A32_UINT:
		case DXGI_FORMAT_R32G32B32A32_SINT:
		case DXGI_FORMAT_BC2_TYPELESS:
		case DXGI_FORMAT_BC2_UNORM:
		case DXGI_FORMAT_BC2_UNORM_SRGB:
		case DXGI_FORMAT_BC3_TYPELESS:
		case DXGI_FORMAT_BC3_UNORM:
		case DXGI_FORMAT_BC3_UNORM_SRGB:
		case DXGI_FORMAT_BC5_TYPELESS:
		case DXGI_FORMAT_BC5_UNORM:
		case DXGI_FORMAT_BC5_SNORM:
		case DXGI_FORMAT_BC6H_TYPELESS:
		case DXGI_FORMAT_BC6H_UF16:
		case DXGI_FORMAT_BC6H_SF16:
		case DXGI_FORMAT_BC7_TYPELESS:
		case DXGI_FORMAT_BC7_UNORM:
		case DXGI_FORMAT_BC7_UNORM_SRGB:
			return 128;

		case DXGI_FORMAT_R32G32B32_TYPELESS:
		case DXGI_FORMAT_R32G32B32_FLOAT:
		case DXGI_FORMAT_R32G32B32_UINT:
		case DXGI_FORMAT_R32G32B32_SINT:
			return 96;

		case DXGI_FORMAT_R16G16B16A16_TYPELESS:
		case DXGI_FORMAT_R16G16B16A16_FLOAT:
		case DXGI_FORMAT_R16G16B16A16_UNORM:
		case DXGI_FORMAT_R16G16B16A16_UINT:
		case DXGI_FORMAT_R16G16B16A16_SNORM:
		case DXGI_FORMAT_R16G16B16A16_SINT:
		case DXGI_FORMAT_R32G32_TYPELESS:
		case DXGI_FORMAT_R32G32_FLOAT:
		case DXGI_FORMAT_R32G32_UINT:
		case DXGI_FORMAT_R32G32_SINT:
		case DXGI_FORMAT_R32G8X24_TYPELESS:
		case DXGI_FORMAT_D32_FLOAT_S8X24_UINT:
		case DXGI_FORMAT_R32_FLOAT_X8X24_TYPELESS:
		case DXGI_FORMAT_X32_TYPELESS_G8X24_UINT:
		case DXGI_FORMAT_BC1_TYPELESS:
		case DXGI_FORMAT_BC1_UNORM:
		case DXGI_FORMAT_BC1_UNORM_SRGB:
		case DXGI_FORMAT_BC4_TYPELESS:
		case DXGI_FORMAT_BC4_UNORM:
		case DXGI_FORMAT_BC4_SNORM:
		case DXGI_FORMAT_Y416:
		case DXGI_FORMAT_Y210:
		case DXGI_FORMAT_Y216:
			return 64;

		case DXGI_FORMAT_R10G10B10A2_TYPELESS:
		case DXGI_FORMAT_R10G10B10A2_UNORM:
		case DXGI_FORMAT_R10G10B10A2_UINT:
		case DXGI_FORMAT_R11G11B10_FLOAT:
		case DXGI_FORMAT_R8G8B8A8_TYPELESS:
		case DXGI_FORMAT_R8G8B8A8_UNORM:
		case DXGI_FORMAT_R8G8B8A8_UNORM_SRGB:
		case DXGI_FORMAT_R8G8B8A8_UINT:
		case DXGI_FORMAT_R8G8B8A8_SNORM:
		case DXGI_FORMAT_R8G8B8A8_SINT:
		case DXGI_FORMAT_R16G16_TYPELESS:
		case DXGI_FORMAT_R16G16_FLOAT:
		case DXGI_FORMAT_R16G16_UNORM:
		case DXGI_FORMAT_R16G16_UINT:
		case DXGI_FORMAT_R16G16_SNORM:
		case DXGI_FORMAT_R16G16_SINT:
		case DXGI_FORMAT_R32_TYPELESS:
		case DXGI_FORMAT_D32_FLOAT:
		case DXGI_FORMAT_R32_FLOAT:
		case DXGI_FORMAT_R32_UINT:
		case DXGI_FORMAT_R32_SINT:
		case DXGI_FORMAT_R24G8_TYPELESS:
		case DXGI_FORMAT_D24_UNORM_S8_UINT:
		case DXGI_FORMAT_R24_UNORM_X8_TYPELESS:
		case DXGI_FORMAT_X24_TYPELESS_G8_UINT:
		case DXGI_FORMAT_R9G9B9E5_SHAREDEXP:
		case DXGI_FORMAT_R8G8_B8G8_UNORM:
		case DXGI_FORMAT_G8R8_G8B8_UNORM:
		case DXGI_FORMAT_B8G8R8A8_UNORM:
		case DXGI_FORMAT_B8G8R8X8_UNORM:
		case DXGI_FORMAT_R10G10B10_XR_BIAS_A2_UNORM:
		case DXGI_FORMAT_B8G8R8A8_TYPELESS:
		case DXGI_FORMAT_B8G8R8A8_UNORM_SRGB:
		case DXGI_FORMAT_B8G8R8X8_TYPELESS:
		case DXGI_FORMAT_B8G8R8X8_UNORM_SRGB:
		case DXGI_FORMAT_AYUV:
		case DXGI_FORMAT_Y410:
		case DXGI_FORMAT_P010:
		case DXGI_FORMAT_P016:
		case DXGI_FORMAT_YUY2:
			return 32;

		case DXGI_FORMAT_R8G8_TYPELESS:
		case DXGI_FORMAT_R8G8_UNORM:
		case DXGI_FORMAT_R8G8_UINT:
		case DXGI_FORMAT_R8G8_SNORM:
		case DXGI_FORMAT_R8G8_SINT:
		case DXGI_FORMAT_R16_TYPELESS:
		case DXGI_FORMAT_R16_FLOAT:
		case DXGI_FORMAT_D16_UNORM:
		case DXGI_FORMAT_R16_UNORM:
		case DXGI_FORMAT_R16_UINT:
		case DXGI_FORMAT_R16_SNORM:
		case DXGI_FORMAT_R16_SINT:
		case DXGI_FORMAT_B5G6R5_UNORM:
		case DXGI_FORMAT_B5G5R5A1_UNORM:
		case DXGI_FORMAT_B4G4R4A4_UNORM:
		case DXGI_FORMAT_NV12:
		case DXGI_FORMAT_NV11:
			return 16;

		case DXGI_FORMAT_R8_TYPELESS:
		case DXGI_FORMAT_R8_UNORM:
		case DXGI_FORMAT_R8_UINT:
		case DXGI_FORMAT_R8_SNORM:
		case DXGI_FORMAT_R8_SINT:
		case DXGI_FORMAT_A8_UNORM:
			return 8;

		case DXGI_FORMAT_R1_UNORM:
			return 1;
		}
	}

	inline uint32 GetFormatSizeInBytes(DXGI_FORMAT Format)
	{
		return GetFormatSizeInBits(Format) / 8;
	}

	inline DXGI_FORMAT GetSwapChainFormat(EPixelFormat PixelFormat)
	{
		DXGI_FORMAT	DXFormat = static_cast<DXGI_FORMAT>(GPixelFormats[PixelFormat].PlatformFormat);
		switch (DXFormat)
		{
		case DXGI_FORMAT_B8G8R8A8_TYPELESS:		return DXGI_FORMAT_B8G8R8A8_UNORM;
		case DXGI_FORMAT_BC1_TYPELESS:			return DXGI_FORMAT_BC1_UNORM;
		case DXGI_FORMAT_BC2_TYPELESS:			return DXGI_FORMAT_BC2_UNORM;
		case DXGI_FORMAT_BC3_TYPELESS:			return DXGI_FORMAT_BC3_UNORM;
		case DXGI_FORMAT_R16_TYPELESS:			return DXGI_FORMAT_R16_UNORM;
		case DXGI_FORMAT_R8G8B8A8_TYPELESS:		return DXGI_FORMAT_R8G8B8A8_UNORM;
		default: 								return DXFormat;
		}
	}
} // UE::DXGIUtilities

#if PLATFORM_WINDOWS

class FRHIFrameFlipStatTrackingRunnable : public FRunnable
{
	FRunnableThread* Thread = nullptr;
	TRefCountPtr<IDXGISwapChain1> SwapChain1;
	bool bInitialized = false;
	bool bRun = false;

	FRHIFrameFlipStatTrackingRunnable(const FRHIFrameFlipStatTrackingRunnable&) = delete;
	FRHIFrameFlipStatTrackingRunnable& operator=(const FRHIFrameFlipStatTrackingRunnable&) = delete;
	FRHIFrameFlipStatTrackingRunnable(FRHIFrameFlipStatTrackingRunnable&&) = delete;
	FRHIFrameFlipStatTrackingRunnable& operator=(FRHIFrameFlipStatTrackingRunnable&&) = delete;

	FRHIFrameFlipStatTrackingRunnable(TRefCountPtr<IDXGISwapChain1> InSwapChain1);
	~FRHIFrameFlipStatTrackingRunnable();

	RHICORE_API virtual uint32 Run() override;
	RHICORE_API virtual void Stop() override;
	
	RHICORE_API void StartThread();
	RHICORE_API void StopThread();

public:
	// Refcounted process-wide singleton. First Create spins up the tracker thread; the matching final Release tears it down.
	// One tracker is sufficient because RHIFrameInfoSetVsyncDetails is a global sink, and WaitForVBlank fires per monitor vblank regardless of swap chain count.
	static RHICORE_API void Create(TRefCountPtr<IDXGISwapChain1> InSwapChain1);
	static RHICORE_API void Release();
};

#endif //PLATFORM_WINDOWS

#endif // PLATFORM_MICROSOFT
