// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ResourceArray.h"
#include "RHIBufferInitializer.h"
#include "RHICore.h"
#include "RHICommandList.h"
#include "RHICoreInitializerCommon.h"

namespace UE::RHICore
{
	// Buffer initializer that just returns the buffer on finalize.
	struct FDefaultBufferInitializer : public FRHIBufferInitializer
	{
		FDefaultBufferInitializer(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, void* InWritableData, uint64 InWritableSize)
			: FRHIBufferInitializer(RHICmdList, Buffer, InWritableData, InWritableSize,
				[Buffer = TRefCountPtr<FRHIBuffer>(Buffer)](FRHICommandListBase&) mutable
				{
					return MoveTemp(Buffer);
				}
			)
		{
		}

		FDefaultBufferInitializer(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer)
			: FDefaultBufferInitializer(RHICmdList, Buffer, nullptr, 0)
		{
		}
	};

	// Buffer initializer that calls Lock on creation and Unlock on finalize.
	struct FLockBufferInitializer : public FRHIBufferInitializer
	{
		FLockBufferInitializer(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer)
			: FRHIBufferInitializer(
				RHICmdList,
				Buffer,
				RHICmdList.LockBuffer(Buffer, 0, Buffer->GetDesc().Size, RLM_WriteOnly),
				Buffer->GetDesc().Size,
				[Buffer = TRefCountPtr<FRHIBuffer>(Buffer)](FRHICommandListBase& RHICmdList) mutable
				{
					RHICmdList.UnlockBuffer(Buffer);
					return MoveTemp(Buffer);
				})
		{
		}
	};

	// Buffer initializer with custom implementation. This type is necessary for access to the FRHIBufferInitializer protected constructor.
	struct FCustomBufferInitializer : public FRHIBufferInitializer
	{
		FCustomBufferInitializer(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, void* InWritableData, uint64 InWritableSize, FFinalizeCallback&& Func)
			: FRHIBufferInitializer(RHICmdList, Buffer, InWritableData, InWritableSize, Forward<FFinalizeCallback>(Func))
		{
		}
	};

	class FInvalidBuffer : public FRHIBuffer
	{
	public:
		FInvalidBuffer(const FRHIBufferCreateDesc& CreateDesc)
			: FRHIBuffer(CreateDesc)
		{
		}
	};

	static FRHIBufferInitializer HandleUnknownBufferInitializerInitAction(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& CreateDesc)
	{
		UE_LOGF(LogRHICore, Fatal, "Unknown or unhandled ERHIBufferInitAction: %d", static_cast<uint32>(CreateDesc.InitAction));

		FInvalidBuffer* Buffer = new FInvalidBuffer(CreateDesc);
		return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
	}

	// Create a buffer initializer for a unified memory platform. Any init actions not handled before here will use default implementations.
	static FRHIBufferInitializer CreateUnifiedMemoryBufferInitializer(
		FRHICommandListBase& RHICmdList
		, const FRHIBufferCreateDesc& CreateDesc
		, FRHIBuffer* Buffer
		, void* WritableData
	)
	{
		if (CreateDesc.InitAction == ERHIBufferInitAction::Default)
		{
			// Use the default buffer implementation, so just return the buffer on finalize.
			return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
		}

		if (CreateDesc.InitAction == ERHIBufferInitAction::ResourceArray)
		{
			// Write the data from the resource array immediately, discard the resource array and then return the buffer on finalize.
			check(CreateDesc.InitialData);

			FMemory::Memcpy(WritableData, CreateDesc.InitialData->GetResourceData(), CreateDesc.InitialData->GetResourceDataSize());

			// Discard the resource array's contents.
			CreateDesc.InitialData->Discard();

			return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
		}

		if (CreateDesc.InitAction == ERHIBufferInitAction::Zeroed)
		{
			// Zero memory immediately and return the buffer on finalize
			FMemory::Memzero(WritableData, CreateDesc.Size);
			return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
		}

		// Let the caller fill in the writable data.
		if (CreateDesc.InitAction == ERHIBufferInitAction::Initializer)
		{
			return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer, WritableData, CreateDesc.Size);
		}

		return UE::RHICore::HandleUnknownBufferInitializerInitAction(RHICmdList, CreateDesc);
	}
}
