// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"

namespace UE::RHICore
{
	// Utility class for RHIs to use when they need to malloc aligned data up-front and make sure it's freed correctly by the Initializer lambda destruction
	struct FInitializerScopedMemory : public FNoncopyable
	{
		FInitializerScopedMemory(void* InPointer)
			: Pointer(InPointer)
		{
		}
		FInitializerScopedMemory(size_t InSize, uint32 InAlignment)
			: FInitializerScopedMemory(FMemory::Malloc(InSize, InAlignment))
		{
		}
		FInitializerScopedMemory(FInitializerScopedMemory&& InOther)
			: Pointer(InOther.Pointer)
		{
			InOther.Pointer = nullptr;
		}
		~FInitializerScopedMemory()
		{
			if (Pointer)
			{
				FMemory::Free(Pointer);
			}
		}

		void* Pointer{};
	};
}
