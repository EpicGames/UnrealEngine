// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIResources.h"
#include "RHIResourceCollection.h"
#include "RHICommandList.h"
#include "DynamicRHI.h"
#include "Containers/ResourceArray.h"

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING

namespace UE::RHICore
{
	template<typename TType>
	inline size_t CalculateResourceCollectionMemorySize(TConstArrayView<TType> InValues)
	{
		return (1 + InValues.Num()) * sizeof(uint32);
	}

	inline FRHIDescriptorHandle GetHandleForResourceCollectionValue(const FRHIResourceCollectionMember& Member)
	{
		switch (Member.Type)
		{
		case FRHIResourceCollectionMember::EType::Texture:
			return static_cast<const FRHITexture*>(Member.Resource)->GetDefaultBindlessHandle();
		case FRHIResourceCollectionMember::EType::TextureReference:
			return static_cast<const FRHITextureReference*>(Member.Resource)->GetBindlessHandle();
		case FRHIResourceCollectionMember::EType::ShaderResourceView:
			return static_cast<const FRHIShaderResourceView*>(Member.Resource)->GetBindlessHandle();
		}

		return FRHIDescriptorHandle();
	}

	inline FRHIDescriptorHandle GetHandleForResourceCollectionValue(const FRHIDescriptorHandle& Handle)
	{
		return Handle;
	}

	inline void FillResourceCollectionUpdateMemory(uint32* Destination, TConstArrayView<FRHIResourceCollectionMember> InValues)
	{
		int32 WriteIndex = 0;

		for (const FRHIResourceCollectionMember& Value : InValues)
		{
			const FRHIDescriptorHandle Handle = GetHandleForResourceCollectionValue(Value);
			check(Handle.IsValid());

			const uint32 BindlessIndex = Handle.IsValid() ? Handle.GetIndex() : 0;

			Destination[WriteIndex] = BindlessIndex;
			++WriteIndex;
		}
	}

	inline void FillResourceCollectionMemory(uint32* Destination, TConstArrayView<FRHIResourceCollectionMember> InValues)
	{
		Destination[0] = static_cast<uint32>(InValues.Num());

		FillResourceCollectionUpdateMemory(Destination + 1, InValues);
	}

	inline void FillResourceCollectionMemory(TRHIBufferInitializer<uint32>& Destination, TConstArrayView<FRHIResourceCollectionMember> InValues)
	{
		FillResourceCollectionMemory(Destination.GetWritableData(), InValues);
	}

	inline FRHIBuffer* CreateResourceCollectionBuffer(FRHICommandListBase& RHICmdList, TConstArrayView<FRHIResourceCollectionMember> InMembers)
	{
		const size_t BufferSize = UE::RHICore::CalculateResourceCollectionMemorySize(InMembers);

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateByteAddress(TEXT("ResourceCollection"), BufferSize, sizeof(uint32))
			.AddUsage(EBufferUsageFlags::Static)
			.SetInitialState(ERHIAccess::SRVMask)
			.SetInitActionInitializer();

		TRHIBufferInitializer<uint32> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
		UE::RHICore::FillResourceCollectionMemory(Initializer, InMembers);

		return Initializer.Finalize();
	}

	class FGenericResourceCollection : public FRHIResourceCollection
	{
	public:
		FGenericResourceCollection(FRHICommandListBase& RHICmdList, TConstArrayView<FRHIResourceCollectionMember> InMembers)
			: FRHIResourceCollection(InMembers)
			, Buffer(CreateResourceCollectionBuffer(RHICmdList, InMembers))
		{
			FRHIViewDesc::FBufferSRV::FInitializer ViewDesc = FRHIViewDesc::CreateBufferSRV().SetType(FRHIViewDesc::EBufferType::Raw);
			ShaderResourceView = RHICmdList.CreateShaderResourceView(Buffer, ViewDesc);
		}

		~FGenericResourceCollection() = default;

		// FRHIResourceCollection
		virtual FRHIDescriptorHandle GetBindlessHandle() const final
		{
			return ShaderResourceView->GetBindlessHandle();
		}
		//~FRHIResourceCollection

		FRHIShaderResourceView* GetShaderResourceView() const
		{
			return ShaderResourceView;
		}

		TRefCountPtr<FRHIBuffer> Buffer;
		TRefCountPtr<FRHIShaderResourceView> ShaderResourceView;
	};

	inline FRHIResourceCollectionRef CreateGenericResourceCollection(FRHICommandListBase& RHICmdList, TConstArrayView<FRHIResourceCollectionMember> InMembers)
	{
		return new FGenericResourceCollection(RHICmdList, InMembers);
	}

	inline TConstArrayView<FRHIResourceCollectionMember> GetValidResourceCollectionUpdateList(FRHIResourceCollection* ResourceCollection, uint32 StartIndex, TConstArrayView<FRHIResourceCollectionMember> InUpdates)
	{
		const uint32 CollectionSize = ResourceCollection->GetMembers().Num();
		if (StartIndex >= CollectionSize)
		{
			// Invalid start index
			return {};
		}

		const int32 UpdateCount = FMath::Min<int32>(InUpdates.Num(), CollectionSize - StartIndex);
		return InUpdates.Left(UpdateCount);
	}

	inline void UpdateGenericResourceCollection(FRHICommandListBase& RHICmdList, FGenericResourceCollection* ResourceCollection, uint32 StartIndex, TConstArrayView<FRHIResourceCollectionMember> InMemberUpdates)
	{
		TConstArrayView<FRHIResourceCollectionMember> MemberUpdates = UE::RHICore::GetValidResourceCollectionUpdateList(ResourceCollection, StartIndex, InMemberUpdates);
		if (!MemberUpdates.IsEmpty())
		{
			// We can't reliably lock with offsets, so we will have to update the entire resource.

			ResourceCollection->UpdateMembers(StartIndex, MemberUpdates);
			TConstArrayView<FRHIResourceCollectionMember> Members = ResourceCollection->GetMembers();

			const uint32 UploadSize = UE::RHICore::CalculateResourceCollectionMemorySize(Members);
			uint32* UploadData = reinterpret_cast<uint32*>(RHICmdList.LockBuffer(ResourceCollection->Buffer, 0, UploadSize, RLM_WriteOnly));

			UE::RHICore::FillResourceCollectionMemory(UploadData, Members);

			RHICmdList.UnlockBuffer(ResourceCollection->Buffer);
		}
	}
}

#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING
