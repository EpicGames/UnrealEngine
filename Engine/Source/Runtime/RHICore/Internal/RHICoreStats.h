// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIDefinitions.h"

struct FRHIBufferDesc;
struct FRHITextureDesc;
struct FTextureMemoryStats;

namespace UE::RHICore
{

RHICORE_API void UpdateGlobalTextureStats(ETextureCreateFlags TextureFlags, ETextureDimension Dimension, uint64 TextureSizeInBytes, bool bOnlyStreamableTexturesInTexturePool, bool bAllocating);
RHICORE_API void UpdateGlobalTextureStats(const FRHITextureDesc& TextureDesc, uint64 TextureSizeInBytes, bool bOnlyStreamableTexturesInTexturePool, bool bAllocating);

RHICORE_API void FillBaselineTextureMemoryStats(FTextureMemoryStats& OutStats);

RHICORE_API void UpdateGlobalBufferStats(const FRHIBufferDesc& BufferDesc, int64 BufferSizeDelta);
inline void UpdateGlobalBufferStats(const FRHIBufferDesc& BufferDesc, uint64 BufferSize, bool bAllocating)
{
	const int64 BufferSizeDelta = bAllocating ? static_cast<int64>(BufferSize) : -static_cast<int64>(BufferSize);
	UpdateGlobalBufferStats(BufferDesc, BufferSizeDelta);
}

RHICORE_API void UpdateGlobalUniformBufferStats(int64 BufferSize, bool bAllocating);

RHICORE_API void UpdateReservedResourceStatsOnCommit(int64 CommitDelta, bool bBuffer, bool bCommitting);

} // namespace UE::RHICore
