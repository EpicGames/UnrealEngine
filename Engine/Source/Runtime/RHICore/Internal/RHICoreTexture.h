// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicRHI.h"
#include "RHIResources.h"

namespace UE::RHICore
{
	inline uint32 GetCombinedArrayIndex(const FRHITextureDesc& Desc, uint32 FaceIndex, uint32 ArrayIndex)
	{
		if (Desc.IsTextureCube())
		{
			return FaceIndex + ArrayIndex * 6;
		}

		if (Desc.IsTextureArray())
		{
			return ArrayIndex;
		}

		return 0;
	}

	inline uint32 GetLockArrayIndex(const FRHITextureDesc& Desc, const FRHILockTextureArgs& Arguments)
	{
		return GetCombinedArrayIndex(Desc, Arguments.FaceIndex, Arguments.ArrayIndex);
	}
}
