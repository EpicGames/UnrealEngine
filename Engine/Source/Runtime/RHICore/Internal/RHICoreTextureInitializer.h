// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ResourceArray.h"
#include "RHITextureInitializer.h"
#include "RHICore.h"
#include "RHICommandList.h"
#include "RHICoreInitializerCommon.h"
#include "RHITextureUtils.h"

namespace UE::RHICore
{
	struct FBaseTextureInitializerImplementation : public FRHITextureInitializer
	{
		FBaseTextureInitializerImplementation(FRHICommandListBase& RHICmdList, FRHITexture* InTexture, void* InWritableData, uint64 InWritableSize, FFinalizeCallback&& InFinalizeCallback, FGetSubresourceCallback&& InGetSubresourceCallback)
			: FRHITextureInitializer(RHICmdList, InTexture, InWritableData, InWritableSize, Forward<FFinalizeCallback>(InFinalizeCallback), Forward<FGetSubresourceCallback>(InGetSubresourceCallback))
		{
		}
		FBaseTextureInitializerImplementation(FRHICommandListBase& RHICmdList, FRHITexture* Texture, FFinalizeCallback&& InFinalizeCallback)
			: FRHITextureInitializer(RHICmdList, Texture, nullptr, 0, Forward<FFinalizeCallback>(InFinalizeCallback), nullptr)
		{
		}

		void* GetWritableData() const
		{
			return WritableData;
		}

		uint64 GetWritableSize() const
		{
			return WritableSize;
		}
	};

	// Texture initializer that just returns the texture on finalize.
	struct FDefaultTextureInitializer : public FBaseTextureInitializerImplementation
	{
		FDefaultTextureInitializer(FRHICommandListBase& RHICmdList, FRHITexture* Texture)
			: FBaseTextureInitializerImplementation(RHICmdList, Texture,
				[Texture = TRefCountPtr<FRHITexture>(Texture)](FRHICommandListBase&) mutable
				{
					return MoveTemp(Texture);
				}
			)
		{
		}
	};

	struct FDefaultLayoutTextureInitializer : public FBaseTextureInitializerImplementation
	{
		FDefaultLayoutTextureInitializer(FRHICommandListBase& RHICmdList, FRHITexture* InTexture, void* InMemory, uint64 InMemorySize, FFinalizeCallback&& FinalizeFunc)
			: FBaseTextureInitializerImplementation(RHICmdList, InTexture, InMemory, InMemorySize, Forward<FFinalizeCallback>(FinalizeFunc),
				[Texture = TRefCountPtr<FRHITexture>(InTexture), WritableData = InMemory](FRHITextureInitializer::FSubresourceIndex SubresourceIndex)
				{
					const FRHITextureDesc& TextureDesc = Texture->GetDesc();

					uint64 SubresourceStride = 0;
					uint64 SubresourceSize = 0;
					const uint64 Offset = UE::RHITextureUtils::CalculateSubresourceOffset(TextureDesc, SubresourceIndex.FaceIndex, SubresourceIndex.ArrayIndex, SubresourceIndex.MipIndex, SubresourceStride, SubresourceSize);

					FRHITextureSubresourceInitializer Result{};
					Result.Data = reinterpret_cast<uint8*>(WritableData) + Offset;
					Result.Stride = SubresourceStride;
					Result.Size = SubresourceSize;

					return Result;
				}
			)
		{
		}
	};

	// Texture Initializer that uses Lock/Unlock to upload initial data
	struct FLockTextureInitializer : public FDefaultLayoutTextureInitializer
	{
		FLockTextureInitializer(FRHICommandListBase& RHICmdList, FRHITexture* InTexture, void* InMemory, uint64 InMemorySize)
			: FDefaultLayoutTextureInitializer(RHICmdList, InTexture, InMemory, InMemorySize,
				[Texture = TRefCountPtr<FRHITexture>(InTexture), WritableMemory = FInitializerScopedMemory(InMemory)](FRHICommandListBase& InRHICmdList) mutable
				{
					FRHICommandListImmediate& RHICmdList = InRHICmdList.GetAsImmediate();

					const FRHITextureDesc& TextureDesc = Texture->GetDesc();
					const uint32 FaceCount = TextureDesc.IsTextureCube() ? 6 : 1;

					for (uint32 FaceIndex = 0; FaceIndex < FaceCount; FaceIndex++)
					{
						for (uint32 ArrayIndex = 0; ArrayIndex < TextureDesc.ArraySize; ArrayIndex++)
						{
							for (uint32 MipIndex = 0; MipIndex < TextureDesc.NumMips; MipIndex++)
							{
								uint64 SubresourceStride = 0;
								uint64 SubresourceSize = 0;
								const uint64 Offset = UE::RHITextureUtils::CalculateSubresourceOffset(TextureDesc, FaceIndex, ArrayIndex, MipIndex, SubresourceStride, SubresourceSize);

								const FRHILockTextureArgs LockArgs = FRHILockTextureArgs::LockCubeFace(Texture, FaceIndex, ArrayIndex, MipIndex, RLM_WriteOnly, false);

								FRHILockTextureResult LockResult = RHICmdList.LockTexture(LockArgs);
								check(LockResult.ByteCount <= SubresourceSize);
								check(LockResult.Stride == SubresourceStride);

								FMemory::Memcpy(LockResult.Data, reinterpret_cast<uint8*>(WritableMemory.Pointer) + Offset, SubresourceSize);

								RHICmdList.UnlockTexture(LockArgs);
							}
						}
					}

					return MoveTemp(Texture);
				})
		{
		}
	};

	static FLockTextureInitializer CreateLockTextureInitializer(FRHICommandListBase& RHICmdList, FRHITexture* Texture, const FRHITextureCreateDesc& CreateDesc)
	{
		const uint64 WritableMemorySize = UE::RHITextureUtils::CalculateTextureSize(CreateDesc);
		void* WritableMemory = FMemory::Malloc(WritableMemorySize, 16);

		return FLockTextureInitializer(RHICmdList, Texture, WritableMemory, WritableMemorySize);
	}

	static FRHITextureInitializer HandleUnknownTextureInitializerInitAction(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc)
	{
		UE_LOGF(LogRHICore, Fatal, "Unknown or unhandled ERHITextureInitAction: %d", static_cast<uint32>(CreateDesc.InitAction));

		FRHITexture* Texture = nullptr;
		return UE::RHICore::FDefaultTextureInitializer(RHICmdList, Texture);
	}
}
