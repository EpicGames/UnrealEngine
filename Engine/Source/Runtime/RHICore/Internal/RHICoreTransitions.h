// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHITransition.h"
#include "RHIContext.h"

namespace UE::RHICore
{

struct FResourceState
{
	FResourceState(IRHIComputeContext& Context, ERHIPipeline InSrcPipelines, ERHIPipeline InDstPipelines, const FRHITransitionInfo& Info)
		: AccessBefore(Info.AccessBefore)
		, AccessAfter(Info.AccessAfter)
		, SrcPipelines(InSrcPipelines)
		, DstPipelines(InDstPipelines)
	{
		if (FRHIViewableResource* ViewableResource = GetViewableResource(Info))
		{
			if (NeedToReplaceBeforeInfo())
			{
				SrcPipelines = Context.GetTrackedPipelines(ViewableResource);
				AccessBefore = Context.GetTrackedAccess(ViewableResource);
			}

			if (AccessAfter == ERHIAccess::Unknown)
			{
				AccessAfter = Context.GetTrackedAccess(ViewableResource);
			}

			check(AccessBefore != ERHIAccess::Unknown);
			check(AccessAfter != ERHIAccess::Unknown);
		}
	}

	ERHIAccess AccessBefore;
	ERHIAccess AccessAfter;

	// SrcPipeline may differ from the declared RHI SrcPipeline when CollapseToSinglePipeline is used, in which case this will be ERHIPipeline::All.
	ERHIPipeline SrcPipelines;
	ERHIPipeline DstPipelines;

private:
	bool NeedToReplaceBeforeInfo() const
	{
		if (AccessBefore == ERHIAccess::Unknown)
		{
			return true;
		}
		if (AccessBefore == ERHIAccess::Discard
			&& GRHIGlobals.NeedsTransientDiscardStateTracking)
		{
			return true;
		}
		return false;
	}
};

} //! UE::RHICore