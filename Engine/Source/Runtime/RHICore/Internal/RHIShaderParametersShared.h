// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIShaderParameters.h"

namespace UE::RHICore
{
	inline FRHIDescriptorHandle GetBindlessParameterHandle(const FRHIShaderParameterResource& Parameter)
	{
		if (FRHIResource* Resource = Parameter.Resource)
		{
			switch (Parameter.Type)
			{
			case FRHIShaderParameterResource::EType::Texture:             return static_cast<FRHITexture*>(Resource)->GetDefaultBindlessHandle();
			case FRHIShaderParameterResource::EType::ResourceView:        return static_cast<FRHIShaderResourceView*>(Resource)->GetBindlessHandle();
			case FRHIShaderParameterResource::EType::UnorderedAccessView: return static_cast<FRHIUnorderedAccessView*>(Resource)->GetBindlessHandle();
			case FRHIShaderParameterResource::EType::Sampler:             return static_cast<FRHISamplerState*>(Resource)->GetBindlessHandle();
			case FRHIShaderParameterResource::EType::ResourceCollection:  return static_cast<FRHIResourceCollection*>(Resource)->GetBindlessHandle();
			case FRHIShaderParameterResource::EType::DescriptorRange:
				checkf(false, TEXT("DescriptorRange types do not have a bindless handle, they need to be bound in a specific manner."));
				break;
			default:
				checkf(false, TEXT("Unhandled resource type?"));
				break;
			}
		}

		return FRHIDescriptorHandle();
	}
}
