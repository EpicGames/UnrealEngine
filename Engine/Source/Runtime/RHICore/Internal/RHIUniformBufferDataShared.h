// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphResources.h"

namespace UE::RHICore
{
	struct FUniformDataReader
	{
		FUniformDataReader() = delete;
		FUniformDataReader(const void* InData) : Data(reinterpret_cast<const uint8*>(InData)) { }

		template<typename TResourceOut>
		const TResourceOut& Read(const FRHIUniformBufferResource& InResource) const
		{
			return *reinterpret_cast<const TResourceOut*>(Data + InResource.MemberOffset);
		}

		const uint8* Data;
	};

	inline FRHIDescriptorHandle GetBindlessResourceHandle(FUniformDataReader Reader, const FRHIUniformBufferResource& Resource)
	{
		switch (Resource.MemberType)
		{
		case UBMT_TEXTURE:
		{
			FRHITexture* Texture = Reader.Read<FRHITexture*>(Resource);
			return Texture ? Texture->GetDefaultBindlessHandle() : FRHIDescriptorHandle();
		}
		break;
		case UBMT_SRV:
		{
			FRHIShaderResourceView* ShaderResourceView = Reader.Read<FRHIShaderResourceView*>(Resource);
			return ShaderResourceView ? ShaderResourceView->GetBindlessHandle() : FRHIDescriptorHandle();
		}
		break;
		case UBMT_SAMPLER:
		{
			FRHISamplerState* SamplerState = Reader.Read<FRHISamplerState*>(Resource);
			return SamplerState ? SamplerState->GetBindlessHandle() : FRHIDescriptorHandle();
		}
		break;
		case UBMT_RESOURCE_COLLECTION:
		{
			FRHIResourceCollection* ResourceCollection = Reader.Read<FRHIResourceCollection*>(Resource);
			return ResourceCollection ? ResourceCollection->GetBindlessHandle() : FRHIDescriptorHandle();
		}
		break;
		case UBMT_RDG_TEXTURE:
		{
			FRDGTexture* RDGTexture = Reader.Read<FRDGTexture*>(Resource);
			FRHITexture* Texture = RDGTexture ? RDGTexture->GetRHI() : nullptr;
			return Texture ? Texture->GetDefaultBindlessHandle() : FRHIDescriptorHandle();
		}
		break;
		case UBMT_RDG_TEXTURE_SRV:
		case UBMT_RDG_TEXTURE_NON_PIXEL_SRV:
		case UBMT_RDG_BUFFER_SRV:
		{
			FRDGShaderResourceView* RDGShaderResourceView = Reader.Read<FRDGShaderResourceView*>(Resource);
			if (RDGShaderResourceView)
				RDGShaderResourceView->MarkResourceAsUsed();

			FRHIShaderResourceView* ShaderResourceView = RDGShaderResourceView ? RDGShaderResourceView->GetRHI() : nullptr;
			return ShaderResourceView ? ShaderResourceView->GetBindlessHandle() : FRHIDescriptorHandle();
		}
		break;
		case UBMT_UAV:
		{
			FRHIUnorderedAccessView* UnorderedAccessView = Reader.Read<FRHIUnorderedAccessView*>(Resource);
			return UnorderedAccessView ? UnorderedAccessView->GetBindlessHandle() : FRHIDescriptorHandle();
		}
		break;
		case UBMT_RDG_TEXTURE_UAV:
		case UBMT_RDG_BUFFER_UAV:
		{
			FRDGUnorderedAccessView* RDGUnorderedAccessView = Reader.Read<FRDGUnorderedAccessView*>(Resource);
			FRHIUnorderedAccessView* UnorderedAccessView = RDGUnorderedAccessView ? RDGUnorderedAccessView->GetRHI() : nullptr;
			return UnorderedAccessView ? UnorderedAccessView->GetBindlessHandle() : FRHIDescriptorHandle();
		}
		break;

		default:
			// Do nothing.
			break;
		}
		return FRHIDescriptorHandle();
	}

	inline void UpdateUniformBufferConstants(void* DestinationData, const void* SourceData, const FRHIUniformBufferLayout& Layout, bool bAllowBindless = true)
	{
		check(DestinationData != nullptr);
		check(SourceData != nullptr);

		// First copy wholesale
		FMemory::Memcpy(DestinationData, SourceData, Layout.ConstantBufferSize);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (bAllowBindless)
		{
			FUniformDataReader Reader(SourceData);

			// Then copy indices over
			for (const FRHIUniformBufferResource& Resource : Layout.Resources)
			{
				if (Resource.MemberType == UBMT_DESCRIPTOR_RANGE)
				{
					if (FRHIDescriptorRange* DescriptorRange = Reader.Read<FRHIDescriptorRange*>(Resource))
					{
						FRHIDescriptorRangeShaderData ShaderData = DescriptorRange->GetShaderData();
						FMemory::Memcpy(reinterpret_cast<uint8*>(DestinationData) + Resource.MemberOffset, &ShaderData, sizeof(ShaderData));
					}
				}
				else
				{
					const FRHIDescriptorHandle Handle = GetBindlessResourceHandle(Reader, Resource);
					if (Handle.IsValid())
					{
						const uint32 BindlessIndex = Handle.GetIndex();
						FMemory::Memcpy(reinterpret_cast<uint8*>(DestinationData) + Resource.MemberOffset, &BindlessIndex, sizeof(BindlessIndex));
					}
				}
			}
		}
#endif
	}
}
