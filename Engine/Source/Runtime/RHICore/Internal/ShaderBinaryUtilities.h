// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHICore.h"
#include "Math/Int128.h"

namespace UE::ShaderBinaryUtilities
{
	struct FShaderDebugData
	{
		FString FunctionName;
		FString PDBName;
		FString Hash;
	};
	
#if PLATFORM_SUPPORTS_PRAGMA_PACK
#pragma pack(push, 1)
#endif // PLATFORM_SUPPORTS_PRAGMA_PACK

	namespace DXBC
	{
		struct FHeader
		{
			uint32 Identifier;
			uint32 Ignore[6];
			uint32 ChunkCount;
		};

		struct FChunkHeader
		{
			uint32 Type;
			uint32 Size;
		};
	}
	
	namespace DXIL
	{
		struct FShaderDebugNameInfo
		{
			uint16 Flags;
			uint16 NameLength;
		};

		struct FShaderHashInfo
		{
			uint32 Flags;
			uint8  Digest[16];
		};

		struct FPSVRevision3Info
		{
			uint8  Ignore[48];
			uint32 EntryNameOffset;
		};
	}

#if PLATFORM_SUPPORTS_PRAGMA_PACK
#pragma pack(pop)
#endif // PLATFORM_SUPPORTS_PRAGMA_PACK
	
	struct FParseContext
	{
		FParseContext(const void* Start, const uint64 ByteSize) : Start(static_cast<const uint8*>(Start)), ByteSize(ByteSize)
		{
			
		}

		template<typename T>
		T Get()
		{
			if (Offset + sizeof(T) > ByteSize)
			{
				checkf(false, TEXT("Parsing beyond EOS"));
				return T{};
			}
			
			return *reinterpret_cast<const T*>(Start + Offset);
		}

		template<typename T>
		T Consume()
		{
			if (Offset + sizeof(T) > ByteSize)
			{
				checkf(false, TEXT("Parsing beyond EOS"));
				return T{};
			}
			
			const T* Value = reinterpret_cast<const T*>(Start + Offset);
			Offset += sizeof(T);
			return *Value;
		}

		FParseContext Split(uint64 InOffset) const
		{
			FParseContext ctx = *this;
			ctx.Offset = InOffset;
			return ctx;
		}

		void Skip(uint64 InOffset)
		{
			Offset += InOffset;
		}

		uint64 PendingBytes() const
		{
			return ByteSize - Offset;
		}

		const void* Data() const
		{
			return Start + Offset;
		}

		const uint8* Start = nullptr;
		const uint64 ByteSize = 0;
		uint64 Offset = 0;
	};
	
	static void GetShaderBinaryDebugHashDXBC(const void* ShaderBinary, uint32 ByteSize, FShaderDebugData& OutData)
	{
		if (ByteSize < sizeof(DXBC::FHeader))
		{
			UE_LOGF(LogRHICore, Error, "Shader byte size too small");
			return;
		}

		FParseContext Ctx(ShaderBinary, ByteSize);

		auto Header = Ctx.Consume<DXBC::FHeader>();

		for (uint32 ChunkIndex = 0; ChunkIndex < Header.ChunkCount; ++ChunkIndex)
		{
			uint32 ChunkOffset = Ctx.Consume<uint32>();

			FParseContext ChunkCtx = Ctx.Split(ChunkOffset);

			auto ChunkHeader = ChunkCtx.Consume<DXBC::FChunkHeader>();
			switch (ChunkHeader.Type)
			{
				default:
				{
					continue;
				}
				case 'NDLI':
				{
					auto DXILDebugInfo = ChunkCtx.Consume<DXIL::FShaderDebugNameInfo>();

					const uint32 HashLength = 32 + FCString::Strlen(TEXT(".pdb"));

					if (DXILDebugInfo.NameLength != HashLength)
					{
						UE_LOGF(LogRHICore, Display, "DXIL name length not the expected hash");
						continue;
					}

					if (ChunkCtx.PendingBytes() < HashLength)
					{
						UE_LOGF(LogRHICore, Display, "ILDN block corrupt");
						continue;
					}

					OutData.PDBName = FString::ConstructFromPtrSize(static_cast<const ANSICHAR*>(ChunkCtx.Data()), HashLength);
					break;
				}
				case 'HSAH':
				{
					auto DXILDebugInfo = ChunkCtx.Consume<DXIL::FShaderHashInfo>();
					static_assert(sizeof(FUInt128) == sizeof(DXILDebugInfo.Digest), "Unexpected digest size");
					
					FUInt128 HashBytes;
					FMemory::Memcpy(&HashBytes, DXILDebugInfo.Digest, sizeof(HashBytes));

					OutData.Hash = BytesToHex(reinterpret_cast<const uint8*>(&HashBytes), sizeof(FUInt128));
					break;
				}
				case '0VSP':
				{
					uint32 RevisionByteSize  = ChunkCtx.Consume<uint32>();
					
					if (RevisionByteSize >= sizeof(DXIL::FPSVRevision3Info))
					{
						DXIL::FPSVRevision3Info PSVInfo = ChunkCtx.Get<DXIL::FPSVRevision3Info>();
						ChunkCtx.Skip(RevisionByteSize);

						if (uint32 ResourceCount = ChunkCtx.Consume<uint32>())
						{
							uint32 BindInfoByteCount  = ChunkCtx.Consume<uint32>();
							ChunkCtx.Skip(BindInfoByteCount * ResourceCount);
						}
						
						uint32 StringTableSize  = ChunkCtx.Consume<uint32>();
						if (StringTableSize <= PSVInfo.EntryNameOffset)
						{
							UE_LOGF(LogRHICore, Error, "Pipeline state validation string table too small");
							continue;
						}

						ChunkCtx.Skip(PSVInfo.EntryNameOffset);
						OutData.FunctionName = static_cast<const char*>(ChunkCtx.Data());
					}
						
					break;
				}
			}
		}
	}
}
