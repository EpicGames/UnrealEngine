// Copyright Epic Games, Inc. All Rights Reserved.

#include "DXGIUtilities.h"

#include "RHIStats.h"
#include "RHIUtilities.h"
#include "CoreMinimal.h"

#if PLATFORM_MICROSOFT

const TCHAR* UE::DXGIUtilities::GetFormatString(DXGI_FORMAT Format)
{
	const TCHAR* Result = TEXT("");

#define DXGI_FORMAT_CASE(x) case x: Result = TEXT(#x); break;
	switch (Format)
	{
		DXGI_FORMAT_CASE(DXGI_FORMAT_R8G8B8A8_UNORM)
		DXGI_FORMAT_CASE(DXGI_FORMAT_B8G8R8A8_UNORM)
		DXGI_FORMAT_CASE(DXGI_FORMAT_B8G8R8X8_UNORM)
		DXGI_FORMAT_CASE(DXGI_FORMAT_BC1_UNORM)
		DXGI_FORMAT_CASE(DXGI_FORMAT_BC2_UNORM)
		DXGI_FORMAT_CASE(DXGI_FORMAT_BC3_UNORM)
		DXGI_FORMAT_CASE(DXGI_FORMAT_BC4_UNORM)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R16G16B16A16_FLOAT)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R32G32B32A32_FLOAT)
		DXGI_FORMAT_CASE(DXGI_FORMAT_UNKNOWN)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R8_UNORM)
		DXGI_FORMAT_CASE(DXGI_FORMAT_D32_FLOAT_S8X24_UINT)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R32_FLOAT_X8X24_TYPELESS)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R32G8X24_TYPELESS)
		DXGI_FORMAT_CASE(DXGI_FORMAT_D24_UNORM_S8_UINT)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R24_UNORM_X8_TYPELESS)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R32_FLOAT)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R16G16_UINT)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R16G16_UNORM)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R16G16_SNORM)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R16G16_FLOAT)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R32G32_FLOAT)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R10G10B10A2_UNORM)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R16G16B16A16_UINT)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R8G8_SNORM)
		DXGI_FORMAT_CASE(DXGI_FORMAT_BC5_UNORM)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R1_UNORM)
		DXGI_FORMAT_CASE(DXGI_FORMAT_R8G8B8A8_TYPELESS)
		DXGI_FORMAT_CASE(DXGI_FORMAT_B8G8R8A8_TYPELESS)
		DXGI_FORMAT_CASE(DXGI_FORMAT_BC7_UNORM)
		DXGI_FORMAT_CASE(DXGI_FORMAT_BC6H_UF16)
		default:
			Result = TEXT("");
	}
#undef DXGI_FORMAT_CASE
	return Result;
}

HRESULT UE::DXGIUtilities::GetD3DMemoryStats(IDXGIAdapter* Adapter, FRHIMemoryStats& OutStats)
{
#if PLATFORM_WINDOWS
	SCOPE_CYCLE_COUNTER(STAT_D3DUpdateVideoMemoryStats);

	TRefCountPtr<IDXGIAdapter3> Adapter3;
	HRESULT Res = Adapter->QueryInterface(IID_PPV_ARGS(Adapter3.GetInitReference()));
	if (FAILED(Res))
	{
		return Res;
	}

	DXGI_QUERY_VIDEO_MEMORY_INFO LocalMemoryInfo;
	Res = Adapter3->QueryVideoMemoryInfo(0, DXGI_MEMORY_SEGMENT_GROUP_LOCAL, &LocalMemoryInfo);
	if (FAILED(Res))
	{
		return Res;
	}

	DXGI_QUERY_VIDEO_MEMORY_INFO NonLocalMemoryInfo;
	Res = Adapter3->QueryVideoMemoryInfo(0, DXGI_MEMORY_SEGMENT_GROUP_NON_LOCAL, &NonLocalMemoryInfo);
	if (FAILED(Res))
	{
		return Res;
	}

	// In case of multiple GPUs, use the memory info from the one with the highest local budget.
	if (!GVirtualMGPU)
	{
		for (uint32 Index = 1; Index < GNumExplicitGPUsForRendering; ++Index)
		{
			DXGI_QUERY_VIDEO_MEMORY_INFO TempLocalMemoryInfo;
			Res = Adapter3->QueryVideoMemoryInfo(Index, DXGI_MEMORY_SEGMENT_GROUP_LOCAL, &TempLocalMemoryInfo);
			if (FAILED(Res))
			{
				return Res;
			}

			DXGI_QUERY_VIDEO_MEMORY_INFO TempNonLocalMemoryInfo;
			Res = Adapter3->QueryVideoMemoryInfo(Index, DXGI_MEMORY_SEGMENT_GROUP_NON_LOCAL, &TempNonLocalMemoryInfo);
			if (FAILED(Res))
			{
				return Res;
			}

			if (TempLocalMemoryInfo.Budget > LocalMemoryInfo.Budget)
			{
				LocalMemoryInfo = TempLocalMemoryInfo;
				NonLocalMemoryInfo = TempNonLocalMemoryInfo;
			}
		}
	}

	OutStats.BudgetLocal = LocalMemoryInfo.Budget;
	OutStats.BudgetSystem = NonLocalMemoryInfo.Budget;
	OutStats.UsedLocal = LocalMemoryInfo.CurrentUsage;
	OutStats.UsedSystem = NonLocalMemoryInfo.CurrentUsage;

	// Check if we're over budget.
	if (OutStats.UsedLocal > OutStats.BudgetLocal)
	{
		OutStats.AvailableLocal = 0;
		OutStats.DemotedLocal = OutStats.UsedLocal - OutStats.BudgetLocal;
	}
	else
	{
		OutStats.AvailableLocal = OutStats.BudgetLocal - OutStats.UsedLocal;
		OutStats.DemotedLocal= 0;
	}

	if (OutStats.UsedSystem > OutStats.BudgetSystem)
	{
		OutStats.AvailableSystem = 0;
		OutStats.DemotedSystem = OutStats.UsedSystem - OutStats.BudgetSystem;
	}
	else
	{
		OutStats.AvailableSystem = OutStats.BudgetSystem - OutStats.UsedSystem;
		OutStats.DemotedSystem = 0;
	}

	return S_OK;
#else
	return E_NOINTERFACE;
#endif
}

#if PLATFORM_WINDOWS

FCriticalSection GFlipStatTrackingCS;
FRHIFrameFlipStatTrackingRunnable* GFlipStatTrackingSingleton = nullptr;
int32 GFlipStatTrackingRefCount = 0;

void FRHIFrameFlipStatTrackingRunnable::Create(TRefCountPtr<IDXGISwapChain1> InSwapChain1)
{
	FScopeLock Lock(&GFlipStatTrackingCS);
	if (GFlipStatTrackingRefCount++ == 0)
	{
		if (ensure(GFlipStatTrackingSingleton == nullptr))
		{
			GFlipStatTrackingSingleton = new FRHIFrameFlipStatTrackingRunnable(MoveTemp(InSwapChain1));
		}
	}
}

void FRHIFrameFlipStatTrackingRunnable::Release()
{
	FScopeLock Lock(&GFlipStatTrackingCS);
	if (!ensure(GFlipStatTrackingRefCount > 0))
	{
		return;
	}
	if (--GFlipStatTrackingRefCount == 0)
	{
		delete GFlipStatTrackingSingleton;
		GFlipStatTrackingSingleton = nullptr;
	}
}

FRHIFrameFlipStatTrackingRunnable::FRHIFrameFlipStatTrackingRunnable(TRefCountPtr<IDXGISwapChain1> InSwapChain1)
: SwapChain1(MoveTemp(InSwapChain1))
{
	StartThread();
}

FRHIFrameFlipStatTrackingRunnable::~FRHIFrameFlipStatTrackingRunnable()
{
	StopThread();
}


void FRHIFrameFlipStatTrackingRunnable::StartThread()
{
	check(Thread == nullptr);
	bInitialized = true;
	bRun = true;
	
	Thread = FRunnableThread::Create(this, TEXT("RHIFrameFlipStatTrackingRunnable"), 0, TPri_Normal);
}

void FRHIFrameFlipStatTrackingRunnable::StopThread()
{
	if (!bInitialized)
	{
		return;
	}

	Stop();

	bInitialized = false;

	if (Thread)
	{
		Thread->Kill(true);
		delete Thread;
		Thread = nullptr;
	}
}

uint32 FRHIFrameFlipStatTrackingRunnable::Run()
{
	FRHIFlipDetails LastFlipFrame;
	DXGI_FRAME_STATISTICS LastStats = { 0 };

	struct FVsyncEntry
	{
		uint64 Cycles;
		float Hz;
	};

	if (!SwapChain1.IsValid())
	{
		return 0;
	}

	TRefCountPtr<IDXGIOutput> Output;
	SwapChain1->GetContainingOutput(Output.GetInitReference());

	uint64 LastVSync = 0;
	float Hz = 0.f;
	const int32 VsyncDelayFrames = 12;
	static constexpr double VsyncMatchEpsilonSeconds = 0.0003; // 0.3ms
	const uint64 VsyncMatchEpsilon = (uint64)(VsyncMatchEpsilonSeconds / FPlatformTime::GetSecondsPerCycle64());

	TArray<FVsyncEntry> VsyncQueue;
	VsyncQueue.Reserve(VsyncDelayFrames * 2);
	
	IConsoleVariable* CVarVsyncInfo = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VsyncInformationInsights"));

	while (bRun)
	{
		if (!Output.IsValid())
		{ 
			// Something wrong in the IDXGIOutput, wait and try to reacquire the information from the SwapChain again
			FPlatformProcess::Sleep(1.0f);
			SwapChain1->GetContainingOutput(Output.GetInitReference());
		}
			
		if (Output.IsValid())
		{
			// Blocking Wait for the next monitor VBlank, yielding the CPU (not busy-waiting)
			TRACE_CPUPROFILER_EVENT_SCOPE(IDXGIOutput::WaitForVBlank_NoBusyWait);
			HRESULT WaitForVblankHR = Output->WaitForVBlank();

			// Valid IDXGIOutput and successful waiting, compute the stats
			if (WaitForVblankHR == S_OK)
			{
				uint64 VsyncCycles = FPlatformTime::Cycles64();
				uint64 delta = VsyncCycles - LastVSync;
				LastVSync = VsyncCycles;
				//Hz = uint32(1000.0 / FPlatformTime::ToMilliseconds64(delta));
				Hz = (float)(1.0 / (double(delta) * FPlatformTime::GetSecondsPerCycle64()));

				VsyncQueue.Add({ VsyncCycles, Hz });

				// Emit bookmarks for entries that have aged past the delay window
				// Entries that have aged past the delay window they are always at the front of the queue since entries are chronological
				while (VsyncQueue.Num() > VsyncDelayFrames)
				{
#if UE_INPUT_LATENCY_TRACKING
					if ((CVarVsyncInfo) && (CVarVsyncInfo->GetInt()))
					{
						TRACE_BOOKMARK_CYCLES(VsyncQueue[0].Cycles, TEXT("Vsync (%.2f Hz) Frame: Missed"), VsyncQueue[0].Hz);
					}
#endif

					VsyncQueue.RemoveAt(0, EAllowShrinking::No);
				}

				HRESULT GetStatHR;
				DXGI_FRAME_STATISTICS stats = { 0 };

				if (SUCCEEDED(GetStatHR = SwapChain1->GetFrameStatistics(&stats)) && (stats.PresentCount > LastFlipFrame.PresentIndex))
				{
					FRHIFlipDetails NewFlipFrame;
					NewFlipFrame.PresentIndex = stats.PresentCount;
					NewFlipFrame.VBlankTimeInCycles = stats.SyncQPCTime.QuadPart;
					NewFlipFrame.Hz = Hz;

					// Remove the matching entry from the queue so it won't emit the bookmark, a more detailed bookmark is emitted in RHIFrameInfoSetVsyncDetails 
					for (int32 i = 0; i < VsyncQueue.Num(); ++i)
					{
						uint64 EntryCycles = VsyncQueue[i].Cycles;
						uint64 FlipCycles = static_cast<uint64>(NewFlipFrame.VBlankTimeInCycles);

						uint64 Diff = (NewFlipFrame.VBlankTimeInCycles > VsyncQueue[i].Cycles) ? NewFlipFrame.VBlankTimeInCycles - VsyncQueue[i].Cycles : VsyncQueue[i].Cycles - NewFlipFrame.VBlankTimeInCycles;

						if (Diff <= VsyncMatchEpsilon)
						{
							VsyncQueue.RemoveAt(i, EAllowShrinking::No);
							break;
						}
					}

					RHIFrameInfoSetVsyncDetails(NewFlipFrame);

					LastFlipFrame = NewFlipFrame;
					LastStats = stats;
				}
			}
			else
			{
				// Force a reacquire of the IDXGIOutput
				Output = nullptr;
			}
		}
	}

	return 0;
}


void FRHIFrameFlipStatTrackingRunnable::Stop()
{
	bRun = false;
}

#endif // PLATFORM_WINDOWS

#endif // PLATFORM_MICROSOFT
