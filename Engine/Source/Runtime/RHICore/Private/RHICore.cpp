// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHICore.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Modules/ModuleManager.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "RHI.h"
#include "RHIShaderPlatformConfig.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, RHICore);
DEFINE_LOG_CATEGORY(LogRHICore);

namespace UE
{
namespace RHICore
{

void ResolveRenderPassTargets(const FRHIRenderPassInfo& RenderPassInfo, TFunction<void(FResolveTextureInfo)> ResolveFunction)
{
	const auto ResolveTexture = [&](FResolveTextureInfo ResolveInfo)
	{
		if (!ResolveInfo.SourceTexture || !ResolveInfo.DestTexture || ResolveInfo.SourceTexture == ResolveInfo.DestTexture)
		{
			return;
		}

		const FRHITextureDesc& SourceDesc = ResolveInfo.SourceTexture->GetDesc();
		const FRHITextureDesc& DestDesc   = ResolveInfo.DestTexture->GetDesc();

		check(SourceDesc.Format == DestDesc.Format);
		check(SourceDesc.Extent == DestDesc.Extent);
		check(SourceDesc.IsMultisample() && !DestDesc.IsMultisample());
		check(SourceDesc.Format != PF_DepthStencil || (SourceDesc.IsTexture2D() && DestDesc.IsTexture2D()));
		check(!SourceDesc.IsTexture3D() && !DestDesc.IsTexture3D());

		ResolveFunction(ResolveInfo);
	};

	for (int32 Index = 0; Index < MaxSimultaneousRenderTargets; ++Index)
	{
		const auto& RTV = RenderPassInfo.ColorRenderTargets[Index];
		ResolveTexture({ RTV.RenderTarget, RTV.ResolveTarget, RTV.MipIndex, RTV.ArraySlice, RenderPassInfo.ResolveRect });
	}

	const auto& DSV = RenderPassInfo.DepthStencilRenderTarget;
	ResolveTexture({ DSV.DepthStencilTarget, DSV.ResolveTarget, 0, 0, RenderPassInfo.ResolveRect });
}

FRHIViewDesc::EDimension AdjustViewInfoDimensionForNarrowing(const FRHIViewDesc::FTexture::FViewInfo& ViewInfo, const FRHITextureDesc& TextureDesc)
{
	FRHIViewDesc::EDimension ViewInfoDimension = ViewInfo.Dimension;
	// some RHIs do not support creating a 2D view to index a specific slice in a 2D array/texture. 
	// But they do support binding a single slice 2D array as a Texture2D in the shader
	if (ViewInfoDimension == FRHIViewDesc::EDimension::Texture2D && (TextureDesc.IsTextureCube() || TextureDesc.IsTextureArray()))
	{
		ViewInfoDimension = FRHIViewDesc::EDimension::Texture2DArray;
		ensureAlwaysMsgf(ViewInfo.ArrayRange.Num == 1 || TextureDesc.ArraySize == 1, TEXT("Trying to create a 2D SRV with more than 1 element"));
	}

	return ViewInfoDimension;
}

bool AllowVendorDevice()
{
	static const bool bAllowVendorDevice = !FParse::Param(FCommandLine::Get(), TEXT("novendordevice"));
	return bAllowVendorDevice;
}

bool AreRayTracingShadersEnabled()
{
	struct FInit
	{
		bool bEnableRayTracingShaders = true;
		FInit()
		{
			FString ModeString;
			GConfig->GetString(FPlatformProperties::GetRuntimeSettingsClassName(), TEXT("RayTracingMode"), ModeString, GEngineIni);

			if (ModeString.Equals(TEXT("Inline"), ESearchCase::IgnoreCase))
			{
				bEnableRayTracingShaders = false;
			}
			else if (ModeString.Equals(TEXT("Full"), ESearchCase::IgnoreCase))
			{
				bEnableRayTracingShaders = true;
			}
		}

	} static Init;

	return Init.bEnableRayTracingShaders;
}

} //! RHICore
} //! UE

ERHIBindlessConfiguration UE::RHICore::GetBindlessConfigurationOnStartup(EShaderPlatform Platform)
{
	if (!FDataDrivenShaderPlatformInfo::GetSupportsBindless(Platform))
	{
		return ERHIBindlessConfiguration::Disabled;
	}

#if WITH_EDITOR
	if (TOptional<ERHIBindlessConfiguration> ForcedBindlessConfiguration = RHIGetForcedBindlessConfiguration())
	{
		return ForcedBindlessConfiguration.GetValue();
	}
#endif

	ERHIBindlessConfiguration BindlessConfiguration = ERHIBindlessConfiguration::Disabled;

	if (const FConfigValue* ConfigValue = UE::RHIShaderPlatformConfig::GetConfigValueForShaderPlatform(GConfig, Platform, TEXTVIEW("BindlessConfiguration")))
	{
		GetBindlessConfigurationFromString(ConfigValue->GetValue(), BindlessConfiguration);
	}

	return BindlessConfiguration;
}
