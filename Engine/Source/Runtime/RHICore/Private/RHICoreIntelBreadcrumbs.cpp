// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHICoreIntelBreadcrumbs.h"

#if INTEL_GPU_CRASH_DUMPS

#include "RHI.h"
#include "RHICore.h"
#include "HAL/FileManager.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/CommandLine.h"
#include "HAL/PlatformCrashContext.h"

#if PLATFORM_WINDOWS
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif
THIRD_PARTY_INCLUDES_START
#include <igdext.h>
THIRD_PARTY_INCLUDES_END
#if PLATFORM_WINDOWS
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#if PLATFORM_WINDOWS
	#include "Windows/AllowWindowsPlatformTypes.h"
	#include "Windows/HideWindowsPlatformTypes.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogIntelBreadcrumbs, Log, All);

namespace UE::RHICore::Intel::GPUCrashDumps
{
	RHICORE_API TAutoConsoleVariable<int32> CVarIntelCrashDumps(
		TEXT( "r.GPUCrashDebugging.IntelCrashDumps" ),
		0,
		TEXT( "Enable/disable Intel GPU Crash Dumps." ),
		ECVF_ReadOnly
	);

	static TAutoConsoleVariable<int32> CVarIntelCrashDumps_Markers(
		TEXT( "r.GPUCrashDebugging.IntelCrashDumps.Markers" ),
		1,
		TEXT( "Enable event markers in the GPU Crash Dumps." ),
		ECVF_ReadOnly
	);

	static TAutoConsoleVariable<int32> CVarIntelCrashDumps_Callstack(
		TEXT( "r.GPUCrashDebugging.IntelCrashDumps.Callstack" ),
		0,
		TEXT( "Enable callstack capture in the GPU Crash Dumps." ),
		ECVF_ReadOnly
	);

	static TAutoConsoleVariable<int32> CVarIntelCrashDumps_Resources(
		TEXT( "r.GPUCrashDebugging.IntelCrashDumps.ResourceTracking" ),
		0,
		TEXT( "Enable resource tracking in the GPU Crash Dumps." ),
		ECVF_ReadOnly
	);

	static TAutoConsoleVariable<float> CVarIntelCrashDumps_DumpWaitTime(
		TEXT( "r.GPUCrashDebugging.IntelCrashDumps.DumpWaitTime" ),
		10.0f,
		TEXT( "Intel Breadcrumbs GPU crash dumps processing timeout." ),
		ECVF_Default
	);

#if WITH_RHI_BREADCRUMBS
	static TIndirectArray<FRHIBreadcrumb::FBuffer> NameStorage;
#endif

	static INTC_CRASHDUMP_INFO* CrashDumpInfo = nullptr;

	bool IsEnabled()
	{
		return bEnabled;
	}

	static void ResolveMarkerCb(const void* pBuffer, const uint32_t bufferSize, void* pPrivateData, void** ppResolvedBuffer, uint32_t* pResolvedBufferSize)
	{
#if WITH_RHI_BREADCRUMBS
		void* Ptr = nullptr;
		if (bufferSize != sizeof(Ptr))
		{
			*ppResolvedBuffer = const_cast<TCHAR*>(TEXT(""));
			*pResolvedBufferSize = 0;
			return;
		}

		FMemory::Memcpy(&Ptr, pBuffer, sizeof(Ptr));

		FRHIBreadcrumbNode const* Breadcrumb = static_cast<FRHIBreadcrumbNode const*>(Ptr);
		if (Breadcrumb == FRHIBreadcrumbNode::Sentinel)
		{
			*ppResolvedBuffer = const_cast<TCHAR*>(RootNodeName);
			*pResolvedBufferSize = sizeof(RootNodeName);
		}
		else
		{
			// Allocate space to hold the name of this breadcrumb.
			FRHIBreadcrumb::FBuffer* Buffer = new FRHIBreadcrumb::FBuffer;
			NameStorage.Add(Buffer);

			TCHAR const* NameStr = Breadcrumb->GetTCHAR(*Buffer);

			*ppResolvedBuffer = const_cast<TCHAR*>(NameStr);
			*pResolvedBufferSize = (FCString::Strlen(NameStr) + 1) * sizeof(TCHAR); // Include null terminator
		}
#endif
	}

	// Initialize Intel Breadcrumbs GPU Crash Dumps - before device creation
	RHICORE_API void InitializeBeforeDeviceCreation(uint32 DeviceId)
	{
		if (!UE::RHI::ShouldEnableGPUCrashFeature(*CVarIntelCrashDumps, TEXT("intelbreadcrumbs")))
		{
			UE_LOGF(LogIntelBreadcrumbs, Log, "Intel Breadcrumbs is explicitly disabled. Intel Breadcrumbs initialization skipped...");
			return;
		}

		Flags |= ( CVarIntelCrashDumps_Markers->GetInt() > 0 ) ? INTC_GPU_CRASH_FLAG_MARKERS : 0;
		Flags |= ( CVarIntelCrashDumps_Resources->GetInt() > 0 ) ? INTC_GPU_CRASH_FLAG_RESOURCE_TRACKING : 0;
		Flags |= ( CVarIntelCrashDumps_Callstack->GetInt() > 0 ) ? INTC_GPU_CRASH_FLAG_CALL_STACK : 0;

		if( FAILED( INTC_LoadExtensionsLibrary( false, (uint32)EGpuVendorId::Intel, DeviceId ) ) )
		{
			UE_LOGF( LogIntelBreadcrumbs, Log, "Failed to load Intel Extensions Library (Intel Breadcrumbs)" );
			return;
		}

		// Initialize Intel Breadcrumbs GPU Crash Dumps
		HRESULT hr = INTC_EnableGpuCrashDumps(
			(INTC_GPU_CRASH_FLAGS)( Flags ),
			{ 
				nullptr, 
				nullptr,
				&ResolveMarkerCb 
			},
			nullptr
		);
		if (hr != S_OK)
		{
			UE_LOGF(LogIntelBreadcrumbs, Log, "Intel Breadcrumbs enabling failed to enable GPU Crash Dumps [hr=0x%08x].", hr);
			if( hr == E_NOINTERFACE )
			{
				UE_LOGF( LogIntelBreadcrumbs, Log, "Intel Breadcrumbs not implemented in the Intel Extensions library");
			}
			return;
		}

		bEnabled = true;
		UE_LOGF(LogIntelBreadcrumbs, Log, "Intel Breadcrumbs Enabled!");
	}

	static void CollectBreadcrumbNodes()
	{
		using EState = FGPUBreadcrumbCrashData::EState;
		struct FBreadcrumbNode
		{
			TOptional<EState> State{};
			FString Name;
			TArray<FBreadcrumbNode> Children;
		};

		// Create a root node that will hold all events as children. The root itself will be discarded.
		FBreadcrumbNode Root;
		TArray<FBreadcrumbNode*> ParentChain = { &Root };
		FGPUBreadcrumbCrashData CrashData( TEXT( "Intel Breadcrumbs" ) );

		// By that time, the crash dump should be ready to be processed
		if (CrashDumpInfo == nullptr )
		{
			return;
		}
		for( uint32_t section = 0; section < CrashDumpInfo->crashDumpSectionCount; section++ )
		{
			// We are only interested in D3D12 breadcrumbs here
			if( CrashDumpInfo->pCrashDumpSections[ section ].sectionType != INTC_CRASHDUMP_SECTION_TYPE_MARKERS )
			{
				continue;
			}

			INTC_CRASHDUMP_SECTION* Section = &CrashDumpInfo->pCrashDumpSections[ section ];
			ERHIPipeline Pipeline = ERHIPipeline::None;

			// Read pipeline type from the section info
			if( FCString::Strlen( Section->pSectionInfo ) > 0 )
			{
				if( FCString::Strcmp( Section->pSectionInfo, TEXT( "3D Queue" ) ) == 0 )
				{
					Pipeline = ERHIPipeline::Graphics;
				}
				else if( FCString::Strcmp( Section->pSectionInfo, TEXT( "Compute Queue" ) ) == 0 )
				{
					Pipeline = ERHIPipeline::AsyncCompute;
				}
				else
				{
					// Unknown pipeline type
					continue;
				}
			}
			else
			{
				// No pipeline type specified
				continue;
			}

			// Loop through all breadcrumbs in the section
			for ( uint32_t i = 0; i < Section->dataEntryCount; i++ )
			{
				INTC_CRASHDUMP_BREADCRUMB_DATA Breadcrumb = reinterpret_cast<INTC_CRASHDUMP_BREADCRUMB_DATA*>( Section->pDataEntry )[ i ];

				if( (Breadcrumb.eventType & INTC_EVENT_MARKER_BEGIN) && !(Breadcrumb.eventType & INTC_EVENT_MARKER_END))		// Begin marker only
				{
					// This is a begin event, potentially with children events.
					FBreadcrumbNode& BreadcrumbNode = ParentChain.Last()->Children.Emplace_GetRef();
					BreadcrumbNode.Name = Breadcrumb.pMarkerName ? Breadcrumb.pMarkerName : TEXT( "Unknown event" );
					BreadcrumbNode.State = Breadcrumb.completed ? EState::Active : EState::NotStarted;

					ParentChain.Push( &BreadcrumbNode );
				}
				else if( ( Breadcrumb.eventType & INTC_EVENT_MARKER_END ) && !( Breadcrumb.eventType & INTC_EVENT_MARKER_BEGIN ) )	// End marker only
				{
					FBreadcrumbNode* Parent = ParentChain.Pop();
					if( !Parent->State.IsSet() )
					{
						return;
					}

					if( Breadcrumb.completed && Parent->State == EState::Active )
					{
						Parent->State = EState::Finished;
					}
				}
				else // Simple marker (begin|end)
				{
					FBreadcrumbNode& BreadcrumbNode = ParentChain.Last()->Children.Emplace_GetRef();
					BreadcrumbNode.Name = FString::Printf( TEXT( "[%s]" ), Breadcrumb.pMarkerName ? Breadcrumb.pMarkerName : TEXT( "Unknown event" ) );
					BreadcrumbNode.State = Breadcrumb.completed ? EState::Finished : EState::NotStarted;
				}

			}

			// Export the breadcrumb data for this pipeline
			if( !Root.Children.IsEmpty() )
			{
				FGPUBreadcrumbCrashData::FSerializer Serializer;
				for( FBreadcrumbNode const& ActualRoot : Root.Children )
				{
					auto Recurse = [&]( FBreadcrumbNode const& Current, auto& Recurse ) -> void
						{
							Serializer.BeginNode( Current.Name, *Current.State );

							for( FBreadcrumbNode const& Child : Current.Children )
							{
								Recurse( Child, Recurse );
							}

							Serializer.EndNode();
						};
					Recurse( ActualRoot, Recurse );
				}

				// Collect and export breadcrumb data separately as part of the crash payload.
				CrashData.Queues.FindOrAdd( FString::Printf( TEXT( "%s Queue 0" ), *GetRHIPipelineName( Pipeline ) ), Serializer.GetResult() );
			}
		}

		if( CrashData.Queues.Num() )
		{
			FGenericCrashContext::SetGPUBreadcrumbs( MoveTemp( CrashData ) );
		}
	}
	
	static void WriteCrashDump( const INTC_CRASHDUMP_SECTION* Section )
	{
		FString OutputLog;
		TOptional<FString> DumpPath = FPaths::Combine( 
			FPaths::ProjectLogDir(), 
			FString::Printf( TEXT( "%s.%s.intel-gpudmp" ), GDynamicRHI->GetNonValidationRHI()->GetName(), *FDateTime::Now().ToString() ) 
		);

		UE_LOGF( LogIntelBreadcrumbs, Display, "Writing Intel Breadcrumbs [%ls] to %ls...", Section->pSectionInfo, *DumpPath.GetValue() );
		if( FArchive* Writer = IFileManager::Get().CreateFileWriter( *DumpPath.GetValue() ) )
		{
			Writer->Serialize( const_cast<void*>( Section->pDataEntry ), Section->dataEntryCount );
			if( Writer->Close() )
			{
				UE_LOGF( LogIntelBreadcrumbs, Display, "\tIntel Breadcrumbs GPU [%ls] file written!", Section->pSectionInfo )
			}
			else
			{
				UE_LOGF( LogIntelBreadcrumbs, Error, "\tFailed to create file: %ls", *DumpPath.GetValue() );
			}
		}
	}

	RHICORE_API bool OnGPUCrash()
	{
		if (!IsEnabled())
		{
			return false;
		}

		const double StartTime = FPlatformTime::Seconds();
		const double EndTime = StartTime + CVarIntelCrashDumps_DumpWaitTime->GetFloat();
		uint32 NumTries = 0;

		HRESULT hr;
		INTC_CRASHDUMP_STATUS Status;

		// Check GPU crash dump processing status and prepare to retrieve the data
		while (true)
		{
			// Get the status of the GPU crash dump (if status is ready, the dump is ready to be retrieved)
			if( FAILED( hr = INTC_GetGpuCrashDump( Status ) ) )
			{
				UE_LOGF( LogIntelBreadcrumbs, Error, "Intel Breadcrumbs GPU Crash Dump error: HRESULT = 0x%08x", hr );
				return false;
			}

			// If Status is not ready - that should be very rare...
			if( Status == INTC_CRASHDUMP_STATUS_NOT_READY )
			{
				NumTries++;
				if( NumTries == 1 )
				{
					UE_LOGF( LogIntelBreadcrumbs, Warning, "Intel Breadcrumbs GPU Crash Dump not ready..." );
				}

				// Crash dump is still in progress...
				if( FPlatformTime::Seconds() >= EndTime )
				{
					UE_LOGF( LogIntelBreadcrumbs, Warning, "Intel Breadcrumbs GPU Crash Dump processing - timeout!" );
					return false;
				}
				FPlatformProcess::Sleep( 0.01f );
				continue;
			}

			UE_LOGF(LogIntelBreadcrumbs, Warning, "Intel Breadcrumbs GPU Crash Dump processed!" );
			break;
		}
		
		// Retrieve the GPU crash dump data
		if( FAILED( hr = INTC_RetrieveGpuCrashDump( CrashDumpInfo ) ) )
		{
			UE_LOGF( LogIntelBreadcrumbs, Error, "Intel Breadcrumbs GPU Crash Dump retreival error: HRESULT = 0x%08x", hr );
			return false;
		}

		// Check if the crash dump has any data sections
		if( CrashDumpInfo->crashDumpSectionCount == 0 )
		{
			UE_LOGF(LogIntelBreadcrumbs, Error, "Intel Breadcrumbs GPU Crash Dump has no sections!");
			return false;
		}

		// Write binary snapshot of the breadcrumb data to a dump file
		for( uint32 section = 0; section < CrashDumpInfo->crashDumpSectionCount; section++ )
		{
			INTC_CRASHDUMP_SECTION* Section = &CrashDumpInfo->pCrashDumpSections[section];

			// Section containing the binary breadcrumb data
			if( Section->sectionType == INTC_CRASHDUMP_SECTION_TYPE_BLOB )
			{
				WriteCrashDump( Section );
				break;
			}
		}

		// Decode the crash dump
		TArray<TCHAR> Report;
		uint32 ReportSize = 0;

		// Get the decoded text buffer size
		if( INTC_DecodeGpuCrashDump( nullptr, ReportSize ) == S_OK )
		{
			if( ReportSize == 0 )
			{
				UE_LOGF( LogIntelBreadcrumbs, Error, "Intel Breadcrumbs GPU Crash Dump is empty." );
				return true;
			}

			// Copy the decoded text buffer content
			Report.AddUninitialized( ReportSize );
			if( INTC_DecodeGpuCrashDump( Report.GetData(), ReportSize ) == S_OK )
			{
				UE_LOGF( LogIntelBreadcrumbs, Display, "\n%ls", Report.GetData() );
			}
			else
			{
				UE_LOGF( LogIntelBreadcrumbs, Error, "Failed to decode Intel Breadcrumbs GPU Crash Dump." );
			}
		}
		else
		{
			UE_LOGF( LogIntelBreadcrumbs, Error, "Failed to decode Intel Breadcrumbs GPU Crash Dump." );
		}

		CollectBreadcrumbNodes();

		return true;
	}
	
}

#endif // INTEL_GPU_CRASH_DUMPS
