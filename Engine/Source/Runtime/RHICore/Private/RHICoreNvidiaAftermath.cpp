// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHICoreNvidiaAftermath.h"

#if NV_AFTERMATH

#include "RHI.h"
#include "RHICore.h"
#include "ShaderBinaryUtilities.h"
#include "HAL/FileManager.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/CommandLine.h"
#include "HAL/PlatformCrashContext.h"
#include "Misc/ScopeRWLock.h"

#if PLATFORM_WINDOWS
	#include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
	#define VULKAN_H_
	#if PLATFORM_WINDOWS
		#include "d3d12.h"
	#endif // PLATFORM_WINDOWS
	#include "GFSDK_Aftermath.h"
	#include "GFSDK_Aftermath_GpuCrashdump.h"
	#include "GFSDK_Aftermath_GpuCrashDumpDecoding.h"
	#include "GFSDK_Aftermath_GpuCrashDumpEditing.h"
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
	#include "Windows/HideWindowsPlatformTypes.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogNvidiaAftermath, Log, All);

// Aftermath loads nvoglv64.dll on Windows. The enterprise/AWS version of this driver DLL hooks VirtualAlloc in its
// DllMain function, which causes ASAN to abort. We're force-disabling Aftermath on ASAN Windows builds for this reason.
#if PLATFORM_WINDOWS && PLATFORM_HAS_ASAN_INCLUDE
#	if defined(__SANITIZE_ADDRESS__)
#		define AFTERMATH_DISABLE_DUE_TO_ASAN 1
#	elif defined(__has_feature)
#		if __has_feature(address_sanitizer)
#			define AFTERMATH_DISABLE_DUE_TO_ASAN 1
#		endif
#	endif
#endif

#ifndef AFTERMATH_DISABLE_DUE_TO_ASAN
#	define AFTERMATH_DISABLE_DUE_TO_ASAN 0
#endif

namespace UE::RHICore::Nvidia::Aftermath
{
	static TAutoConsoleVariable<int32> CVarAftermath(
		TEXT("r.GPUCrashDebugging.Aftermath"),
		1,
		TEXT("Enables or disables Nvidia Aftermath."),
		ECVF_ReadOnly
	);

	static TAutoConsoleVariable<int32> CVarAftermath_Markers(
		TEXT("r.GPUCrashDebugging.Aftermath.Markers"),
		0,
		TEXT("Enable draw event markers in Aftermath dumps"),
		ECVF_ReadOnly
	);

	static TAutoConsoleVariable<int32> CVarAftermath_Callstack(
		TEXT("r.GPUCrashDebugging.Aftermath.Callstack"),
		0,
		TEXT("Enable callstack capture in Aftermath dumps"),
		ECVF_ReadOnly
	);

	static TAutoConsoleVariable<int32> CVarAftermath_Resources(
		TEXT("r.GPUCrashDebugging.Aftermath.ResourceTracking"),
		0,
		TEXT("Enable resource tracking for Aftermath dumps"),
		ECVF_ReadOnly
	);

	static TAutoConsoleVariable<int32> CVarAftermath_ShaderErrorReporting(
		TEXT("r.GPUCrashDebugging.Aftermath.ShaderErrorReporting"),
		0,
		TEXT("Enable shader error reporting for Aftermath dumps"),
		ECVF_ReadOnly
	);

	static TAutoConsoleVariable<int32> CVarAftermath_TrackAll(
		TEXT("r.GPUCrashDebugging.Aftermath.TrackAll"),
		0,
		TEXT("Enable maximum tracking for Aftermath dumps"),
		ECVF_ReadOnly
	);

	static TAutoConsoleVariable<int> CVarAftermath_ShaderRegistration(
		TEXT("r.GPUCrashDebugging.Aftermath.ShaderRegistration"),
		1,
		TEXT("Enable registration of shaders and pipelines in Aftermath."),
		ECVF_ReadOnly
	);

	static TAutoConsoleVariable<float> CVarAftermath_DumpShaderDebugInfo(
		TEXT("r.GPUCrashDebugging.Aftermath.DumpShaderDebugInfo"),
		0,
		TEXT("Dump shader debug info (.nvdbg) alongside the crash dump."),
		ECVF_ReadOnly
	);

	static TAutoConsoleVariable<float> CVarAftermath_LateShaderAssociationsTimeLimit(
		TEXT("r.GPUCrashDebugging.Aftermath.LateShaderAssociations.TimeLimit"),
		7.5f,
		TEXT("Time limit (s) before the late associations are stopped\n"),
		ECVF_Default
	);

	static TAutoConsoleVariable<int32> CVarAftermath_LateShaderAssociationsFrameLimit(
		TEXT("r.GPUCrashDebugging.Aftermath.LateShaderAssociations.FrameLimit"),
		10,
		TEXT("Max last since used number of frames to consider when collecting pipelines\n"),
		ECVF_Default
	);

	static TAutoConsoleVariable<float> CVarAftermath_DumpProcessWaitTime(
		TEXT("r.GPUCrashDebugging.Aftermath.DumpProcessWaitTime"),
		10.0f,
		TEXT("Amount of time (in seconds) to wait for Aftermath to finish processing GPU crash dumps."),
		ECVF_Default
	);

	static TAutoConsoleVariable<float> CVarAftermath_DumpStartWaitTime(
		TEXT("r.GPUCrashDebugging.Aftermath.DumpStartWaitTime"),
		3.0f,
		TEXT("Amount of time (in seconds) to wait for Aftermath to start processing GPU crash dumps."),
		ECVF_Default
	);

	static void* DllHandle = nullptr;
	static bool bEnabled = false;
	static uint32 Flags = 0;
	static FAssociateShadersFunc GLateShaderAssociateFunc;

	struct FShaderDebugInfo
	{
		FString DebugName;
		ShaderBinaryUtilities::FShaderDebugData DebugData;
	};
	
	struct FShaderRegistration
	{
		FCriticalSection               Lock;
		TMap<uint64, FShaderDebugInfo> ShaderBinaryToDebugHash;
	} static ShaderRegistration;

	struct FCrashIntermediates
	{
		FCriticalSection Lock;

		struct FCrashDump
		{
			TArray<uint8> Data;
		};
		TArray<FCrashDump> CrashDumps;

		TMap<uint64, TArray<uint8>> ShaderDebugLookup;
		std::atomic_uint32_t        Counter{0};
	} static CrashIntermediates;

#if WITH_RHI_BREADCRUMBS == 0
	static constexpr TCHAR const BreadcrumbsDisabledStr[] = TEXT("<RHI breadcrumbs disabled>");
#endif

	RHICORE_API bool IsEnabled() { return bEnabled; }
	RHICORE_API bool IsShaderRegistrationEnabled() { return bEnabled && CVarAftermath_ShaderRegistration->GetInt(); }
	RHICORE_API bool IsResourceTrackingEnabled() { return bEnabled && (Flags & GFSDK_Aftermath_FeatureFlags_EnableResourceTracking); }
	RHICORE_API bool AreMarkersEnabled() { return bEnabled && (Flags & GFSDK_Aftermath_FeatureFlags_EnableMarkers); }

	static void GFSDK_AFTERMATH_CALL Callback_GpuCrashDump    (const void* GPUCrashDumpData, const uint32_t GPUCrashDumpSize, void* UserData);
	static void GFSDK_AFTERMATH_CALL Callback_GpuCrashDumpDesc(PFN_GFSDK_Aftermath_AddGpuCrashDumpDescription AddValue, void* UserData);
	static void GFSDK_AFTERMATH_CALL Callback_ShaderDebugInfo (const void* ShaderDebugInfo, const uint32 ShaderDebugInfoSize, void* UserData);

	// Loads the Aftermath DLL unless Aftermath initialization is disabled.
	void LoadAftermathDLL()
	{
#if AFTERMATH_DISABLE_DUE_TO_ASAN
		UE_LOGF(LogNvidiaAftermath, Log, "Nvidia Aftermath is not compatible with Windows ASAN builds, so it is disabled.");
#else
		if (GDynamicRHI && GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::D3D11)
		{
			UE_LOGF(LogNvidiaAftermath, Log, "Nvidia Aftermath is disabled in D3D11 due to instability issues.");
			return;
		}

		if (!AllowVendorDevice())
		{
			UE_LOGF(LogNvidiaAftermath, Log, "Vendor devices disallowed. Aftermath initialization skipped.");
			return;
		}

		if (!UE::RHI::ShouldEnableGPUCrashFeature(*CVarAftermath, TEXT("nvaftermath")))
		{
			UE_LOGF(LogNvidiaAftermath, Log, "Nvidia Aftermath is explicitly disabled. Aftermath initialization skipped.");
			return;
		}

#if PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
		const TCHAR* AftermathDLLName = TEXT("GFSDK_Aftermath_Lib.arm64.dll");
#else
		const TCHAR* AftermathDLLName = TEXT("GFSDK_Aftermath_Lib.x64.dll");
#endif // PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC

		const FString AftermathBinariesRoot = FPaths::EngineDir() / TEXT("Binaries/ThirdParty/NVIDIA/NVaftermath/Win64/");

		FPlatformProcess::PushDllDirectory(*AftermathBinariesRoot);
		DllHandle = FPlatformProcess::GetDllHandle(AftermathDLLName);
		FPlatformProcess::PopDllDirectory(*AftermathBinariesRoot);

		if (DllHandle == nullptr)
		{
			UE_LOGF(LogNvidiaAftermath, Warning, "Failed to load %ls", AftermathDLLName);
		}
#endif
	}
	
	RHICORE_API void SetLateShaderAssociateCallback(FAssociateShadersFunc ShaderAssociateFunc)
	{
		GLateShaderAssociateFunc = MoveTemp(ShaderAssociateFunc);
	}

	RHICORE_API void InitializeBeforeDeviceCreation()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::RHICore::Nvidia::Aftermath::InitializeBeforeDeviceCreation);

		LoadAftermathDLL();

		if (!DllHandle)
			return;

		static constexpr uint32 AllFlags = 
			  GFSDK_Aftermath_FeatureFlags_EnableMarkers
			| GFSDK_Aftermath_FeatureFlags_EnableResourceTracking
			| GFSDK_Aftermath_FeatureFlags_CallStackCapturing
			| GFSDK_Aftermath_FeatureFlags_GenerateShaderDebugInfo
			| GFSDK_Aftermath_FeatureFlags_EnableShaderErrorReporting
			;

		Flags = GFSDK_Aftermath_FeatureFlags_Minimum;
		Flags |= (FParse::Param(FCommandLine::Get(), TEXT("nvaftermathmarkers"             ))  || (CVarAftermath_Markers             ->GetInt())) ? GFSDK_Aftermath_FeatureFlags_EnableMarkers              : 0;
		Flags |= (FParse::Param(FCommandLine::Get(), TEXT("nvaftermathresources"           ))  || (CVarAftermath_Resources           ->GetInt())) ? GFSDK_Aftermath_FeatureFlags_EnableResourceTracking     : 0;
		Flags |= (FParse::Param(FCommandLine::Get(), TEXT("nvaftermathcallstack"           ))  || (CVarAftermath_Callstack           ->GetInt())) ? GFSDK_Aftermath_FeatureFlags_CallStackCapturing         : 0;
		Flags |= (FParse::Param(FCommandLine::Get(), TEXT("nvAftermathDumpShaderDebugInfo" ))  || (CVarAftermath_DumpShaderDebugInfo ->GetInt())) ? GFSDK_Aftermath_FeatureFlags_GenerateShaderDebugInfo    : 0;
		Flags |= (FParse::Param(FCommandLine::Get(), TEXT("nvAftermathShaderErrorReporting"))  || (CVarAftermath_ShaderErrorReporting->GetInt())) ? GFSDK_Aftermath_FeatureFlags_EnableShaderErrorReporting : 0;
		Flags |= (FParse::Param(FCommandLine::Get(), TEXT("nvaftermathall"                 ))  || (CVarAftermath_TrackAll            ->GetInt())) ? AllFlags                                                : 0;

		GFSDK_Aftermath_GpuCrashDumpWatchedApiFlags WatchedAPIFlags = GFSDK_Aftermath_GpuCrashDumpWatchedApiFlags_DX;
		if (GDynamicRHI && GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::Vulkan)
		{
			WatchedAPIFlags = GFSDK_Aftermath_GpuCrashDumpWatchedApiFlags_Vulkan;
		}

		GFSDK_Aftermath_Result Result = GFSDK_Aftermath_EnableGpuCrashDumps(
			GFSDK_Aftermath_Version_API,
			WatchedAPIFlags,
			GFSDK_Aftermath_GpuCrashDumpFeatureFlags_DeferDebugInfoCallbacks,
			&Callback_GpuCrashDump,
			(Flags & GFSDK_Aftermath_FeatureFlags_GenerateShaderDebugInfo) ? &Callback_ShaderDebugInfo : nullptr,
			&Callback_GpuCrashDumpDesc,
			nullptr, // ResolveMarkerCb - Markers are resolved after dump creation
			nullptr // user data
		);

		if (Result != GFSDK_Aftermath_Result_Success)
		{
			UE_LOGF(LogNvidiaAftermath, Log, "Aftermath crash dumping failed to initialize (%x).", Result);
			return;
		}

		UE_LOGF(LogNvidiaAftermath, Log, "Aftermath initialized");
		bEnabled = true;
	}

	bool InitializeDevice(TFunctionRef<uint32(uint32)> InitCallback)
	{
		ON_SCOPE_EXIT
		{
			FGenericCrashContext::SetEngineData(TEXT("RHI.Aftermath"), bEnabled ? TEXT("true") : TEXT("false"));
		};

		if (!bEnabled)
		{
			UE_LOGF(LogNvidiaAftermath, Log, "Aftermath is not loaded.");
			return false;
		}

		if (!IsRHIDeviceNVIDIA())
		{
			UE_LOGF(LogNvidiaAftermath, Warning, "Skipping aftermath initialization on non-Nvidia device.");
			bEnabled = false;
			return false;
		}

		GFSDK_Aftermath_Result Result = (GFSDK_Aftermath_Result)InitCallback(Flags);
		if (Result != GFSDK_Aftermath_Result_Success)
		{
			UE_LOGF(LogNvidiaAftermath, Log, "Aftermath enabled but failed to initialize (%x).", Result);
			bEnabled = false;
			return false;
		}

		UE_LOGF(LogNvidiaAftermath, Log, "Aftermath enabled. Active feature flags: ");
		for (uint32 LowestBit, Bits = Flags; Bits; Bits ^= LowestBit)
		{
			LowestBit = Bits & (-int32(Bits));

			switch (LowestBit)
			{
			case GFSDK_Aftermath_FeatureFlags_EnableMarkers             : UE_LOGF(LogNvidiaAftermath, Log, " - Feature: EnableMarkers"); break;
			case GFSDK_Aftermath_FeatureFlags_EnableResourceTracking    : UE_LOGF(LogNvidiaAftermath, Log, " - Feature: EnableResourceTracking"); break;
			case GFSDK_Aftermath_FeatureFlags_CallStackCapturing        : UE_LOGF(LogNvidiaAftermath, Log, " - Feature: CallStackCapturing"); break;
			case GFSDK_Aftermath_FeatureFlags_GenerateShaderDebugInfo   : UE_LOGF(LogNvidiaAftermath, Log, " - Feature: GenerateShaderDebugInfo"); break;
			case GFSDK_Aftermath_FeatureFlags_EnableShaderErrorReporting: UE_LOGF(LogNvidiaAftermath, Log, " - Feature: EnableShaderErrorReporting"); break;

			default:
				UE_LOGF(LogNvidiaAftermath, Log, " - Feature: Unknown flag: 0x%08x", LowestBit);
				break;
			}
		}

		return true;
	}

	// Called on NVIDIA driver thread
	static void GFSDK_AFTERMATH_CALL Callback_GpuCrashDump(const void* GPUCrashDumpData, const uint32_t GPUCrashDumpSize, void* UserData)
	{
		FScopeLock Lock(&CrashIntermediates.Lock);

		// Take a copy of the data provided by Aftermath. We process this on the interrupt thread after dumping is complete.
		TArray<uint8>& Data = CrashIntermediates.CrashDumps.Emplace_GetRef().Data;
		Data.SetNumUninitialized(GPUCrashDumpSize);
		FMemory::Memcpy(Data.GetData(), GPUCrashDumpData, GPUCrashDumpSize);
	}

	// Called on NVIDIA driver thread
	static void GFSDK_AFTERMATH_CALL Callback_GpuCrashDumpDesc(PFN_GFSDK_Aftermath_AddGpuCrashDumpDescription AddValue, void* UserData)
	{
		AddValue(GFSDK_Aftermath_GpuCrashDumpDescriptionKey_ApplicationName, TCHAR_TO_UTF8(*FPlatformCrashContext::GetCrashGameName()));
		AddValue(GFSDK_Aftermath_GpuCrashDumpDescriptionKey_ApplicationVersion, TCHAR_TO_UTF8(FApp::GetBuildVersion()));

		FPlatformCrashContext CrashContext(ECrashContextType::GPUCrash, TEXT("Aftermath crash dump"));
		CrashContext.SerializeContentToBuffer();
		const FString& Buffer = CrashContext.GetBuffer();
		AddValue(GFSDK_Aftermath_GpuCrashDumpDescriptionKey_UserDefined + 0, TCHAR_TO_UTF8(*Buffer));
	}

	// Called on NVIDIA driver thread
	void Callback_ShaderDebugInfo(const void* ShaderDebugInfo, const uint32 ShaderDebugInfoSize, void* UserData)
	{
		// Get shader debug information identifier.
		GFSDK_Aftermath_ShaderDebugInfoIdentifier Identifier = {};
		GFSDK_Aftermath_Result Result = GFSDK_Aftermath_GetShaderDebugInfoIdentifier(GFSDK_Aftermath_Version_API, ShaderDebugInfo, ShaderDebugInfoSize, &Identifier);

		if (Result == GFSDK_Aftermath_Result_Success)
		{
			// Write to file for later in-depth analysis of crash dumps with Nsight Graphics.
			const FString Filename = FPaths::ProjectLogDir() / *FString::Printf(TEXT("%016llX-%016llX.nvdbg"), Identifier.id[0], Identifier.id[1]);
			TUniquePtr<FArchive> Writer = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*Filename));
			if (Writer)
			{
				Writer->Serialize((void*)ShaderDebugInfo, ShaderDebugInfoSize);
				Writer->Close();
			}

			// Make sure the crash handler picks up the file
			FGenericCrashContext::AddFile(Filename);

			// Keep around for json dumping
			FScopeLock Lock(&CrashIntermediates.Lock);
			CrashIntermediates.ShaderDebugLookup.Add(Identifier.id[0], TArray(static_cast<const uint8*>(ShaderDebugInfo), ShaderDebugInfoSize));
		}
	}

	static void AppendGpuCrashShaderBinaryInfo(GFSDK_Aftermath_GpuCrashDump_Decoder Decoder, const GFSDK_Aftermath_GpuCrashDump_ShaderInfo& Shader, FCrashResult& CrashResult)
	{
		GFSDK_Aftermath_ShaderBinaryHash BinaryHash;

		GFSDK_Aftermath_Result Result = GFSDK_Aftermath_GetShaderHashForShaderInfo(Decoder, &Shader, &BinaryHash);
		if (Result != GFSDK_Aftermath_Result_Success)
		{
			CrashResult.OutputLog += FString::Printf(TEXT("\n\t\t\t! Failed to get binary hash (%u)"), static_cast<uint32>(Result));
			return;
		}

		// Report binary hash separately, may not be the same
		CrashResult.OutputLog += FString::Printf(TEXT("\n\t\t\tAftermath Binary Hash = %llu"), BinaryHash.hash);

		FShaderDebugInfo* Candidate = ShaderRegistration.ShaderBinaryToDebugHash.Find(BinaryHash.hash);
		if (!Candidate)
		{
			CrashResult.OutputLog += TEXT("\n\t\t\t! Failed to associate Aftermath hash");
			return;
		}

		if (!Candidate->DebugName.IsEmpty())
		{
			CrashResult.OutputLog += FString::Printf(TEXT("\n\t\t\tDebugName = '%s'"), *Candidate->DebugName);
		}

		if (!Candidate->DebugData.FunctionName.IsEmpty())
		{
			CrashResult.OutputLog += FString::Printf(TEXT("\n\t\t\tFunction = %s"), *Candidate->DebugData.FunctionName);
		}

		if (!Candidate->DebugData.PDBName.IsEmpty())
		{
			CrashResult.OutputLog += FString::Printf(TEXT("\n\t\t\tShader PDB = %s"), *Candidate->DebugData.PDBName);
		}

		if (!Candidate->DebugData.Hash.IsEmpty())
		{
			CrashResult.OutputLog += FString::Printf(TEXT("\n\t\t\tShader Hash = %s"), *Candidate->DebugData.Hash);
		}
	}

	static void AppendGpuCrashShaderInfo(GFSDK_Aftermath_GpuCrashDump_Decoder Decoder, FCrashResult& CrashResult)
	{
		uint32 ShaderCount = 0;

		FScopeLock Lock(&ShaderRegistration.Lock);
		
		// Total number of faulting shaders
		GFSDK_Aftermath_Result Result = GFSDK_Aftermath_GpuCrashDump_GetActiveShadersInfoCount(Decoder, &ShaderCount);
		if (Result != GFSDK_Aftermath_Result_Success)
		{
			CrashResult.OutputLog += FString::Printf(TEXT("\n\t\tFailed to get shader count (%u)."), static_cast<uint32>(Result));
			return;
		}
		
		if (!ShaderCount)
		{
			CrashResult.OutputLog += TEXT("\n\t\tNo Shaders.");
			return;
		}
		
		TArray<GFSDK_Aftermath_GpuCrashDump_ShaderInfo> Shaders;
		Shaders.SetNumUninitialized(ShaderCount);

		// Get all faulting shaders
		Result = GFSDK_Aftermath_GpuCrashDump_GetActiveShadersInfo(Decoder, ShaderCount, Shaders.GetData());
		if (Result != GFSDK_Aftermath_Result_Success)
		{
			CrashResult.OutputLog += FString::Printf(TEXT("\n\t\tFailed to get shaders (%u)."), static_cast<uint32>(Result));
			return;
		}

		CrashResult.OutputLog += FString::Printf(TEXT("\n\t\t%u total."), ShaderCount);

		for (uint32 ShaderIndex = 0; ShaderIndex < ShaderCount; ++ShaderIndex)
		{
			CrashResult.OutputLog += FString::Printf(TEXT("\n\t\t[%u]:"), ShaderIndex);

			const GFSDK_Aftermath_GpuCrashDump_ShaderInfo& Shader = Shaders[ShaderIndex];
			
			if (Shader.isInternal)
			{
				CrashResult.OutputLog += TEXT("\n\t\t\t! Internal");
			}

			// Translate faulting shader type
			const TCHAR* ShaderType = nullptr;
			switch (Shader.shaderType)
			{
			default:
			case GFSDK_Aftermath_ShaderType_Unknown:
				ShaderType = TEXT("Unknown");
				break;
			case GFSDK_Aftermath_ShaderType_Vertex:
				ShaderType = TEXT("Vertex");
				break;
			case GFSDK_Aftermath_ShaderType_Tessellation_Control:
				ShaderType = TEXT("Tessellation Control / Hull");
				break;
			case GFSDK_Aftermath_ShaderType_Tessellation_Evaluation:
				ShaderType = TEXT("Tessellation Evaluation / Domain");
				break;
			case GFSDK_Aftermath_ShaderType_Geometry:
				ShaderType = TEXT("Geometry");
				break;
			case GFSDK_Aftermath_ShaderType_Fragment:
				ShaderType = TEXT("Pixel");
				break;
			case GFSDK_Aftermath_ShaderType_Compute:
				ShaderType = TEXT("Compute");
				break;
			case GFSDK_Aftermath_ShaderType_RayTracing_RayGeneration:
				ShaderType = TEXT("RayTracing Generation");
				break;
			case GFSDK_Aftermath_ShaderType_RayTracing_Miss:
				ShaderType = TEXT("RayTracing Miss");
				break;
			case GFSDK_Aftermath_ShaderType_RayTracing_Intersection:
				ShaderType = TEXT("RayTracing Intersection");
				break;
			case GFSDK_Aftermath_ShaderType_RayTracing_AnyHit:
				ShaderType = TEXT("RayTracing AnyHit");
				break;
			case GFSDK_Aftermath_ShaderType_RayTracing_ClosestHit:
				ShaderType = TEXT("RayTracing ClosestHit");
				break;
			case GFSDK_Aftermath_ShaderType_RayTracing_Callable:
				ShaderType = TEXT("RayTracing Callable");
				break;
			case GFSDK_Aftermath_ShaderType_RayTracing_Internal:
				ShaderType = TEXT("RayTracing Internal");
				break;
			case GFSDK_Aftermath_ShaderType_Mesh:
				ShaderType = TEXT("Mesh");
				break;
			case GFSDK_Aftermath_ShaderType_Task:
				ShaderType = TEXT("Task");
				break;
			}
			
			CrashResult.OutputLog += FString::Printf(TEXT("\n\t\t\tType = %s"), ShaderType);
			CrashResult.OutputLog += FString::Printf(TEXT("\n\t\t\tAftermath Hash = %llu"), Shader.shaderHash);

			AppendGpuCrashShaderBinaryInfo(Decoder, Shader, CrashResult);
		}
	}

	static const TCHAR* ResidencyStateToString(GFSDK_Aftermath_ResourceResidency State)
	{
		switch (State)
		{
		case GFSDK_Aftermath_ResourceResidency_FullyResident: return TEXT("Resident");
		case GFSDK_Aftermath_ResourceResidency_Evicted: return TEXT("Evicted");
		case GFSDK_Aftermath_ResourceResidency_MemoryFreed: return TEXT("Memory Freed");
		case GFSDK_Aftermath_ResourceResidency_MemoryUnbound: return TEXT("Memory Unbound");
		case GFSDK_Aftermath_ResourceResidency_Unknown:
		default:
			return TEXT("Unknown");
		}
	}

	static void ProcessCrashDump(TArray<FCrashResult>& OutResults, const void* GPUCrashDumpData, const uint32_t GPUCrashDumpSize)
	{
		FCrashResult& CrashResult = OutResults.Emplace_GetRef();

		// Write out crash dump to project log dir
		CrashResult.DumpPath = FPaths::Combine(FPaths::ProjectLogDir(), FString::Printf(
			TEXT("%s.%u.%s.nv-gpudmp"),
			GDynamicRHI->GetNonValidationRHI()->GetName(),
			CrashIntermediates.Counter++,
			*FDateTime::Now().ToString()
		));
		FGenericCrashContext::AddFile(*CrashResult.DumpPath);
		
		CrashResult.OutputLog += FString::Printf(TEXT("\n\nWriting Aftermath dump to: %s"), *CrashResult.DumpPath.GetValue());
		if (TUniquePtr<FArchive> Writer = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*CrashResult.DumpPath.GetValue())))
		{
			Writer->Serialize(const_cast<void*>(GPUCrashDumpData), GPUCrashDumpSize);
			Writer->Close();

			CrashResult.OutputLog += TEXT("\n\tSucceeded in writing Aftermath dump file.");
		}
		else
		{
			CrashResult.OutputLog += FString::Printf(TEXT("\n\tFailed to create dump file: %s"), *CrashResult.DumpPath.GetValue());
			CrashResult.DumpPath.Reset();
		}

		CrashResult.OutputLog += TEXT("\n\nDecoding Aftermath GPU Crash:");

		GFSDK_Aftermath_GpuCrashDump_Decoder Decoder;
		GFSDK_Aftermath_Result Result = GFSDK_Aftermath_GpuCrashDump_CreateDecoder(GFSDK_Aftermath_Version_API, GPUCrashDumpData, GPUCrashDumpSize, &Decoder);
		if (Result != GFSDK_Aftermath_Result_Success)
		{
			CrashResult.OutputLog += FString::Printf(TEXT("\n\n\tFailed to create a GPU crash dump decoder object: 0x%08x. No further GPU crash dump information available."), Result);
			return;
		}

		ON_SCOPE_EXIT
		{
			GFSDK_Aftermath_GpuCrashDump_DestroyDecoder(Decoder);
		};

		// Log the overall device status
		CrashResult.OutputLog += TEXT("\n\n\tDevice Info:");
		{
			GFSDK_Aftermath_GpuCrashDump_DeviceInfo DeviceInfo {};
			Result = GFSDK_Aftermath_GpuCrashDump_GetDeviceInfo(Decoder, &DeviceInfo);
			if (Result != GFSDK_Aftermath_Result_Success)
			{
				CrashResult.OutputLog += FString::Printf(TEXT("\n\t\tFailed to retrieve device info: 0x%08x"), Result);
			}
			else
			{
				FString StatusStr;
				switch (DeviceInfo.status)
				{
				case GFSDK_Aftermath_Device_Status_Active     : StatusStr = TEXT("Active"     ); break;
				case GFSDK_Aftermath_Device_Status_Timeout    : StatusStr = TEXT("Timeout"    ); break;
				case GFSDK_Aftermath_Device_Status_OutOfMemory: StatusStr = TEXT("OutOfMemory"); break;
				case GFSDK_Aftermath_Device_Status_PageFault  : StatusStr = TEXT("PageFault"  ); break;
				case GFSDK_Aftermath_Device_Status_Stopped    : StatusStr = TEXT("Stopped"    ); break;
				case GFSDK_Aftermath_Device_Status_Reset      : StatusStr = TEXT("Reset"      ); break;
				case GFSDK_Aftermath_Device_Status_Unknown    : StatusStr = TEXT("Unknown"    ); break;
				case GFSDK_Aftermath_Device_Status_DmaFault   : StatusStr = TEXT("DmaFault"   ); break;
				default:
					StatusStr = FString::Printf(TEXT("Unknown (0x%08x)"), DeviceInfo.status);
					break;
				}

				CrashResult.OutputLog += FString::Printf(
					TEXT("\n\t\tStatus       : %s")
					TEXT("\n\t\tAdapter Reset: %s")
					TEXT("\n\t\tEngine Reset : %s"),
					*StatusStr,
					DeviceInfo.adapterReset ? TEXT("True") : TEXT("False"),
					DeviceInfo.engineReset  ? TEXT("True") : TEXT("False")
				);
			}
		}

		// Append Page Fault info
		CrashResult.OutputLog += TEXT("\n\n\t Page Fault Info:");
		{
			GFSDK_Aftermath_GpuCrashDump_PageFaultInfo FaultInfo;
			Result = GFSDK_Aftermath_GpuCrashDump_GetPageFaultInfo(Decoder, &FaultInfo);
			if (Result == GFSDK_Aftermath_Result_NotAvailable)
			{
				CrashResult.OutputLog += TEXT("\n\t\tNo information on faulting address.");
			}
			else if (Result != GFSDK_Aftermath_Result_Success)
			{
				CrashResult.OutputLog += FString::Printf(TEXT("\n\t\tFailed to retrieve page fault info (Result: 0x%08x)"), Result);
			}
			else
			{
				FString FaultType;
				switch (FaultInfo.faultType)
				{
				case GFSDK_Aftermath_FaultType_AddressTranslationError: FaultType = TEXT("AddressTranslationError"); break;
				case GFSDK_Aftermath_FaultType_IllegalAccessError     : FaultType = TEXT("IllegalAccessError"     ); break;
				default:
					FaultType = FString::Printf(TEXT("Unknown (0x%08x)"), FaultInfo.faultType);
					break;
				}

				FString AccessType;
				switch (FaultInfo.accessType)
				{
				case GFSDK_Aftermath_AccessType_Unknown: AccessType = TEXT("Unknown"); break;
				case GFSDK_Aftermath_AccessType_Read   : AccessType = TEXT("Read"   ); break;
				case GFSDK_Aftermath_AccessType_Write  : AccessType = TEXT("Write"  ); break;
				case GFSDK_Aftermath_AccessType_Atomic : AccessType = TEXT("Atomic" ); break;
				default:
					AccessType = FString::Printf(TEXT("Unknown (0x%08x)"), FaultInfo.accessType);
					break;
				}

				FString Engine;
				switch (FaultInfo.engine)
				{
				case GFSDK_Aftermath_Engine_Unknown        : Engine = TEXT("Unknown"        ); break;
				case GFSDK_Aftermath_Engine_Graphics       : Engine = TEXT("Graphics"       ); break;
				case GFSDK_Aftermath_Engine_GraphicsCompute: Engine = TEXT("GraphicsCompute"); break;
				case GFSDK_Aftermath_Engine_Display        : Engine = TEXT("Display"        ); break;
				case GFSDK_Aftermath_Engine_CopyEngine     : Engine = TEXT("CopyEngine"     ); break;
				case GFSDK_Aftermath_Engine_VideoDecoder   : Engine = TEXT("VideoDecoder"   ); break;
				case GFSDK_Aftermath_Engine_VideoEncoder   : Engine = TEXT("VideoEncoder"   ); break;
				case GFSDK_Aftermath_Engine_Other          : Engine = TEXT("Other"          ); break;
				default:
					Engine = FString::Printf(TEXT("Unknown (0x%08x)"), FaultInfo.engine);
					break;
				}

				FString Client;
				switch (FaultInfo.client)
				{
				case GFSDK_Aftermath_Client_Unknown                  : Client = TEXT("Unknown"                  ); break;
				case GFSDK_Aftermath_Client_HostInterface            : Client = TEXT("HostInterface"            ); break;
				case GFSDK_Aftermath_Client_FrontEnd                 : Client = TEXT("FrontEnd"                 ); break;
				case GFSDK_Aftermath_Client_PrimitiveDistributor     : Client = TEXT("PrimitiveDistributor"     ); break;
				case GFSDK_Aftermath_Client_GraphicsProcessingCluster: Client = TEXT("GraphicsProcessingCluster"); break;
				case GFSDK_Aftermath_Client_PolymorphEngine          : Client = TEXT("PolymorphEngine"          ); break;
				case GFSDK_Aftermath_Client_RasterEngine             : Client = TEXT("RasterEngine"             ); break;
				case GFSDK_Aftermath_Client_Rasterizer2D             : Client = TEXT("Rasterizer2D"             ); break;
				case GFSDK_Aftermath_Client_RenderOutputUnit         : Client = TEXT("RenderOutputUnit"         ); break;
				case GFSDK_Aftermath_Client_TextureProcessingCluster : Client = TEXT("TextureProcessingCluster" ); break;
				case GFSDK_Aftermath_Client_CopyEngine               : Client = TEXT("CopyEngine"               ); break;
				case GFSDK_Aftermath_Client_VideoDecoder             : Client = TEXT("VideoDecoder"             ); break;
				case GFSDK_Aftermath_Client_VideoEncoder             : Client = TEXT("VideoEncoder"             ); break;
				case GFSDK_Aftermath_Client_Other                    : Client = TEXT("Other"                    ); break;
				default:
					Client = FString::Printf(TEXT("Unknown (0x%08x)"), FaultInfo.client);
					break;
				}

				CrashResult.OutputLog += FString::Printf(
					TEXT("\n\t\tGPU VA  : 0x%016llx")
					TEXT("\n\t\tType    : %s")
					TEXT("\n\t\tAccess  : %s")
					TEXT("\n\t\tEngine  : %s")
					TEXT("\n\t\tClient  : %s"),
					FaultInfo.faultingGpuVA,
					*FaultType,
					*AccessType,
					*Engine,
					*Client);

				if (FaultInfo.resourceInfoCount > 0)
				{
					TArray<GFSDK_Aftermath_GpuCrashDump_ResourceInfo> FaultResources;
					FaultResources.SetNum(FaultInfo.resourceInfoCount);
					GFSDK_Aftermath_GpuCrashDump_GetPageFaultResourceInfo(Decoder, FaultInfo.resourceInfoCount, &FaultResources[0]);

					for (uint32 ResIdx = 0; ResIdx < FaultInfo.resourceInfoCount; ++ResIdx)
					{
						const GFSDK_Aftermath_GpuCrashDump_ResourceInfo& FaultResInfo = FaultResources[ResIdx];
						CrashResult.OutputLog += FString::Printf(
							TEXT("\n\t\tResource %d/%d:")
							TEXT("\n\t\t\tName                : '%hs'")
							TEXT("\n\t\t\tGPU VA              : 0x%016llx")
							TEXT("\n\t\t\tSize                : 0x%016llx")
							TEXT("\n\t\t\tWidth               : %d")
							TEXT("\n\t\t\tHeight              : %d")
							TEXT("\n\t\t\tDepth               : %d")
							TEXT("\n\t\t\tMipLevels           : %d")
							TEXT("\n\t\t\tFormat              : %d")
							TEXT("\n\t\t\tIs Buffer           : %s")
							TEXT("\n\t\t\tIs Texture Heap     : %s")
							TEXT("\n\t\t\tIs RTV/DSV Heap     : %s")
							TEXT("\n\t\t\tPlaced Resource     : %s")
							TEXT("\n\t\t\tResidency           : %s")
							TEXT("\n\t\t\tWas Destroyed       : %s")
							TEXT("\n\t\t\tCreate/Destroy Count: %d"),
							ResIdx,
							FaultInfo.resourceInfoCount,
							FaultResInfo.debugName,
							FaultResInfo.gpuVa,
							FaultResInfo.size,
							FaultResInfo.width,
							FaultResInfo.height,
							FaultResInfo.depth,
							FaultResInfo.mipLevels,
							FaultResInfo.format,
							FaultResInfo.bIsBufferHeap ? TEXT("True") : TEXT("False"),
							FaultResInfo.bIsStaticTextureHeap ? TEXT("True") : TEXT("False"),
							FaultResInfo.bIsRenderTargetOrDepthStencilViewHeap ? TEXT("True") : TEXT("False"),
							FaultResInfo.bPlacedResource ? TEXT("True") : TEXT("False"),
							ResidencyStateToString(FaultResInfo.residency),
							FaultResInfo.bWasDestroyed ? TEXT("True") : TEXT("False"),
							FaultResInfo.createDestroyTickCount
						);
					}
				}
				else
				{
					CrashResult.OutputLog += TEXT("\n\tResource: <no info>");
				}

				if (FaultInfo.faultingGpuVA)
				{
					CrashResult.GPUFaultAddress = FaultInfo.faultingGpuVA;
				}
			}
		}

		// Append marker information
		CrashResult.OutputLog += TEXT("\n\n\tMarker Data:");
		{
			uint32 MarkerCount;
			Result = GFSDK_Aftermath_GpuCrashDump_GetEventMarkersInfoCount(Decoder, &MarkerCount);
			if (Result != GFSDK_Aftermath_Result_Success || MarkerCount == 0)
			{
				CrashResult.OutputLog += TEXT("\n\t\tNo marker info.");
			}
			else
			{
				TArray<GFSDK_Aftermath_GpuCrashDump_EventMarkerInfo> MarkerInfos;
				MarkerInfos.AddUninitialized(MarkerCount);
				Result = GFSDK_Aftermath_GpuCrashDump_GetEventMarkersInfo(Decoder, MarkerInfos.Num(), MarkerInfos.GetData());
				if (Result != GFSDK_Aftermath_Result_Success)
				{
					CrashResult.OutputLog += FString::Printf(TEXT("\n\t\tFailed to retrieve marker info array (0x%08x)."), Result);
				}
				else
				{
					for (GFSDK_Aftermath_GpuCrashDump_EventMarkerInfo const& Marker : MarkerInfos)
					{
						TCHAR const* Status;
						switch (Marker.contextStatus)
						{
						case GFSDK_Aftermath_Context_Status_NotStarted: Status = TEXT("Not Started"); break;
						case GFSDK_Aftermath_Context_Status_Executing : Status = TEXT("Executing  "); break;
						case GFSDK_Aftermath_Context_Status_Finished  : Status = TEXT("Finished   "); break;
						case GFSDK_Aftermath_Context_Status_Invalid   : Status = TEXT("Invalid    "); break;
						default                                       : Status = TEXT("Unknown    "); break;
						}

						TCHAR const* Type;
						switch (Marker.contextType)
						{
						case GFSDK_Aftermath_Context_Type_Invalid     : Type = TEXT("Invalid     "); break;
						case GFSDK_Aftermath_Context_Type_Immediate   : Type = TEXT("Immediate   "); break;
						case GFSDK_Aftermath_Context_Type_CommandList : Type = TEXT("CommandList "); break;
						case GFSDK_Aftermath_Context_Type_Bundle      : Type = TEXT("Bundle      "); break;
						case GFSDK_Aftermath_Context_Type_CommandQueue: Type = TEXT("CommandQueue"); break;
						default                                       : Type = TEXT("Unknown     "); break;
						}

					#if WITH_RHI_BREADCRUMBS
						FRHIBreadcrumb::FBuffer Buffer;
					#endif

						FString Name;
						switch (Marker.markerDataOwnership)
						{
						case GFSDK_Aftermath_EventMarkerDataOwnership_User:
							{
						#if WITH_RHI_BREADCRUMBS
								// User-owned markers are pointers to RHI breadcrumb nodes
								FRHIBreadcrumbNode const* Node = static_cast<FRHIBreadcrumbNode const*>(Marker.markerData);
								Name = Node == FRHIBreadcrumbNode::Sentinel
									? RootNodeName
									: Node->GetTCHAR(Buffer);
						#else
							Name = BreadcrumbsDisabledStr;
						#endif
							}
							break;

						case GFSDK_Aftermath_EventMarkerDataOwnership_Decoder:
							// Decoder-owned markers are raw strings
							Name = static_cast<TCHAR const*>(Marker.markerData);
							break;
						}

						CrashResult.OutputLog += FString::Printf(TEXT("\n\t\t[0x%016llx, %s]: [%s] %s"), Marker.contextId, Type, Status, *Name);
					}
				}
			}
		}

		// Append shader information
		CrashResult.OutputLog += TEXT("\n\n\tActive Shaders:");
		{
			// Allow association after a fault
			if (GLateShaderAssociateFunc)
			{
				GLateShaderAssociateFunc(
					CVarAftermath_LateShaderAssociationsTimeLimit->GetFloat(),
					CVarAftermath_LateShaderAssociationsFrameLimit->GetInt()
				);
			}
			
			AppendGpuCrashShaderInfo(Decoder, CrashResult);
		}

		// Append JSON dump generated by the decoder.
		CrashResult.OutputLog += TEXT("\n\n\tJSON Data:");
		{
			auto ShaderDebugInfoLookup = [](const GFSDK_Aftermath_ShaderDebugInfoIdentifier* Identifier, PFN_GFSDK_Aftermath_SetData SetShaderDebugInfo, void* UserData)
			{
				TArray<uint8>* DebugData = CrashIntermediates.ShaderDebugLookup.Find(Identifier->id[0]);
				if (!DebugData)
				{
					UE_LOGF(LogRHICore, Error, "Shader debug data not found (%llu)", Identifier->id[0]);
					return;
				}

				SetShaderDebugInfo(DebugData->GetData(), DebugData->Num());
			};

			uint32 JsonSize = 0;
			Result = GFSDK_Aftermath_GpuCrashDump_GenerateJSON(
				Decoder,
				GFSDK_Aftermath_GpuCrashDumpDecoderFlags_ALL_INFO,
				GFSDK_Aftermath_GpuCrashDumpFormatterFlags_UTF8_OUTPUT,
				ShaderDebugInfoLookup,
				nullptr,
				nullptr,
				nullptr,
				&JsonSize);

			// Cleanup
			CrashIntermediates.ShaderDebugLookup.Empty();

			if (Result != GFSDK_Aftermath_Result_Success)
			{
				CrashResult.OutputLog += FString::Printf(TEXT("\n\t\tFailed to generate JSON from GPU crash dump: 0x%08x"), Result);
				return;
			}

			TArray<UTF8CHAR> JsonStr;
			JsonStr.AddUninitialized(JsonSize);

			Result = GFSDK_Aftermath_GpuCrashDump_GetJSON(Decoder, JsonSize, (char*)JsonStr.GetData());
			if (Result != GFSDK_Aftermath_Result_Success)
			{
				CrashResult.OutputLog += FString::Printf(TEXT("\n\t\tFailed to get JSON string from GPU crash decoder: 0x%08x"), Result);
			}
			else
			{
				CrashResult.OutputLog += FString::Printf(TEXT("\n\n%s\n\n"), StringCast<TCHAR>(JsonStr.GetData()).Get());
			}
		}

		// Report the decoded results, this is to avoid cases where Aftermath fails to collect data post crash handling.
		UE_LOGF(LogNvidiaAftermath, Error, "%ls", *CrashResult.OutputLog);
	}

	FShaderHash RegisterShaderBinary(const void* ShaderBinary, uint32 ByteSize, const FStringView& DebugName)
	{
		if (ByteSize < sizeof(uint32))
		{
			UE_LOGF(LogRHICore, Display, "Shader byte size too small, expected magic dword");
			return InvalidShaderHash;
		}

		FShaderDebugInfo DebugInfo;

		if (!DebugName.IsEmpty())
		{
			DebugInfo.DebugName = DebugName;
		}

		// Internal aftermath hash
		GFSDK_Aftermath_ShaderBinaryHash AftermathHash{};
		
		switch (*static_cast<const uint32*>(ShaderBinary))
		{
			default:
			{
				UE_LOGF(LogRHICore, Display, "Unknown shader format");
				return InvalidShaderHash;
			}
			case 'CBXD': // DXBC container header
			{
				ShaderBinaryUtilities::GetShaderBinaryDebugHashDXBC(ShaderBinary, ByteSize, DebugInfo.DebugData);
					
#ifdef __d3d12_h__
				D3D12_SHADER_BYTECODE ByteCode;
				ByteCode.pShaderBytecode = ShaderBinary;
				ByteCode.BytecodeLength = ByteSize;

				GFSDK_Aftermath_Result Result = GFSDK_Aftermath_GetShaderHash(GFSDK_Aftermath_Version_API, &ByteCode, &AftermathHash);
				if (Result != GFSDK_Aftermath_Result_Success)
				{
					UE_LOGF(LogNvidiaAftermath, Display, "Failed to get aftermath shader hash (%u)", Result);
					return InvalidShaderHash;
				}
#else // __d3d12_h__
				UE_LOGF(LogNvidiaAftermath, Display, "DXBC container without D3D12 support");
				return InvalidShaderHash;
#endif // __d3d12_h__
					
				break;
			}
			case 0x07230203: // Vulkan magic header
			{
				// Consider extracting DebugStoragePath from NonSemantic.Shader.DebugInfo.100
					
#ifdef VULKAN_H_
				GFSDK_Aftermath_SpirvCode SPIRV;
				SPIRV.pData = ShaderBinary;
				SPIRV.size = ByteSize;
					
				GFSDK_Aftermath_Result Result = GFSDK_Aftermath_GetShaderHashSpirv(GFSDK_Aftermath_Version_API, &SPIRV, &AftermathHash);
				if (Result != GFSDK_Aftermath_Result_Success)
				{
					UE_LOGF(LogNvidiaAftermath, Display, "Failed to get aftermath shader hash (%u)", Result);
					return InvalidShaderHash;
				}
#else // VULKAN_H_
				UE_LOGF(LogNvidiaAftermath, Display, "SPIRV container without Vulkan support");
				return InvalidShaderHash;
#endif // VULKAN_H_
			}
		}

		FScopeLock Lock(&ShaderRegistration.Lock);
		ShaderRegistration.ShaderBinaryToDebugHash.Add(AftermathHash.hash, DebugInfo);
		
		return AftermathHash.hash;
	}
	
	void DeregisterShaderBinary(FShaderHash Hash)
	{
		FScopeLock Lock(&ShaderRegistration.Lock);
		ShaderRegistration.ShaderBinaryToDebugHash.Remove(Hash);
	}
	
	bool OnGPUCrash(TArray<FCrashResult>& OutResults)
	{
		if (!bEnabled)
			return false;

		const double StartTime = FPlatformTime::Seconds();
		const double ProcessingEndTime = StartTime + CVarAftermath_DumpProcessWaitTime->GetFloat();
		const double NotStartedEndTime = StartTime + CVarAftermath_DumpStartWaitTime->GetFloat();

		GFSDK_Aftermath_Result Result;

		// Wait for GPU crash dump processing to complete.
		while (true)
		{
			GFSDK_Aftermath_CrashDump_Status Status;
			if (Result = GFSDK_Aftermath_GetCrashDumpStatus(&Status); Result != GFSDK_Aftermath_Result_Success)
			{
				UE_LOGF(LogNvidiaAftermath, Warning, "GFSDK_Aftermath_GetCrashDumpStatus failed: 0x%08x. Skipping crash dump processing.", Result);
				return false;
			}

			switch (Status)
			{
				default: [[fallthrough]];
				case GFSDK_Aftermath_CrashDump_Status_Unknown:
				{
					UE_LOGF(LogNvidiaAftermath, Warning, "Unknown aftermath crash dump state. Skipping crash dump processing.");
					return false;
				}

				case GFSDK_Aftermath_CrashDump_Status_NotStarted:
				{
					// Driver may take time before starting the dump
					if (FPlatformTime::Seconds() >= NotStartedEndTime)
					{
						UE_LOGF(LogNvidiaAftermath, Warning, "Timed out while waiting for Aftermath to start the GPU crash dump.");
						return false;
					}
					else
					{
						FPlatformProcess::Sleep(0.01f);
					}
					break;
				}

				case GFSDK_Aftermath_CrashDump_Status_CollectingData:
				case GFSDK_Aftermath_CrashDump_Status_InvokingCallback:
				{
					// Crash dump is still in progress...
					if (FPlatformTime::Seconds() >= ProcessingEndTime)
					{
						UE_LOGF(LogNvidiaAftermath, Warning, "Timed out while waiting for Aftermath to complete GPU crash dump.");
						return false;
					}
					else
					{
						FPlatformProcess::Sleep(0.01f);
					}
					break;
				}

				case GFSDK_Aftermath_CrashDump_Status_CollectingDataFailed:
				{
					UE_LOGF(LogNvidiaAftermath, Warning, "Aftermath failed to collect GPU crash data.");
					return false;
				}

				case GFSDK_Aftermath_CrashDump_Status_Finished:
					goto DumpReady;
			}
		}

	DumpReady:
		FScopeLock Lock(&CrashIntermediates.Lock);

		for (auto& CrashDump : CrashIntermediates.CrashDumps)
		{
			// Use a crash dump editor to insert the resolved marker names
			GFSDK_Aftermath_GpuCrashDump_Editor Editor {};
			Result = GFSDK_Aftermath_GpuCrashDump_CreateEditor(
				GFSDK_Aftermath_Version_API,
				CrashDump.Data.GetData(),
				CrashDump.Data.Num(),
				&Editor);

			if (Result != GFSDK_Aftermath_Result_Success)
			{
				UE_LOGF(LogNvidiaAftermath, Warning, "GFSDK_Aftermath_GpuCrashDump_CreateEditor failed: 0x%08x. Skipping crash dump processing.", Result);
				return false;
			}

			ON_SCOPE_EXIT
			{
				GFSDK_Aftermath_GpuCrashDump_DestroyEditor(Editor);
			};

			auto ResolveMarker = [](const void* MarkerData, const uint32_t MarkerDataSize, void* UserData, PFN_GFSDK_Aftermath_ResolveMarker ResolveMarker)
			{
			#if WITH_RHI_BREADCRUMBS
				FRHIBreadcrumbNode const* Breadcrumb = static_cast<FRHIBreadcrumbNode const*>(MarkerData);
				if (Breadcrumb == FRHIBreadcrumbNode::Sentinel)
				{
					ResolveMarker(RootNodeName, sizeof(RootNodeName));
				}
				else
				{
					// Retrieve the breadcrumb formatted string
					FRHIBreadcrumb::FBuffer Buffer;
					TCHAR const* NameStr = Breadcrumb->GetTCHAR(Buffer);
					uint32 NameLen = FCString::Strlen(NameStr) + 1; // Include null terminator

					ResolveMarker(NameStr, NameLen * sizeof(TCHAR));
				}
			#else
				ResolveMarker(BreadcrumbsDisabledStr, sizeof(BreadcrumbsDisabledStr));
			#endif
			};

			if (Result = GFSDK_Aftermath_GpuCrashDumpEditor_ResolveEventMarkers(Editor, ResolveMarker, nullptr); Result != GFSDK_Aftermath_Result_Success)
			{
				UE_LOGF(LogNvidiaAftermath, Warning, "GFSDK_Aftermath_GpuCrashDumpEditor_ResolveEventMarkers failed: 0x%08x. Skipping crash dump processing.", Result);
				return false;
			}

			// Retrieve the edited crash dump data
			uint32 CrashDumpSize = 0;
			if (Result = GFSDK_Aftermath_GpuCrashDumpEditor_GetCrashDumpData(Editor, nullptr, 0, &CrashDumpSize); Result != GFSDK_Aftermath_Result_Success)
			{
				UE_LOGF(LogNvidiaAftermath, Warning, "GFSDK_Aftermath_GpuCrashDumpEditor_GetCrashDumpData failed: 0x%08x. Skipping crash dump processing.", Result);
				return false;
			}

			TArray<uint8> FinalDumpData;
			FinalDumpData.SetNumUninitialized(CrashDumpSize);
			if (Result = GFSDK_Aftermath_GpuCrashDumpEditor_GetCrashDumpData(Editor, FinalDumpData.GetData(), FinalDumpData.Num(), &CrashDumpSize); Result != GFSDK_Aftermath_Result_Success)
			{
				UE_LOGF(LogNvidiaAftermath, Warning, "GFSDK_Aftermath_GpuCrashDumpEditor_GetCrashDumpData failed: 0x%08x. Skipping crash dump processing.", Result);
				return false;
			}

			ProcessCrashDump(OutResults, FinalDumpData.GetData(), FinalDumpData.Num());
		}

		// Any valid dump?
		const bool bHasDump = OutResults.ContainsByPredicate([](const FCrashResult& Result)
		{
			return Result.DumpPath.IsSet();
		});

		FGenericCrashContext::SetEngineData(TEXT("RHI.HasAftermathCrashDump"), bHasDump ? TEXT("true") : TEXT("false"));
		return true;
	}
}

#endif // NV_AFTERMATH
