// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHICoreShader.h"
#include "PipelineStateCache.h"

#define RHI_VALIDATE_STATIC_UNIFORM_BUFFERS (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)

void UE::RHICore::ValidateStaticUniformBuffer(FRHIUniformBuffer* UniformBuffer, FUniformBufferStaticSlot Slot, uint32 ExpectedHash)
{
#if RHI_VALIDATE_STATIC_UNIFORM_BUFFERS
	FUniformBufferStaticSlotRegistry& SlotRegistry = FUniformBufferStaticSlotRegistry::Get();

	if (!UniformBuffer)
	{
		const FShaderParametersMetadata* ExpectedStructMetadata = FindUniformBufferStructByLayoutHash(ExpectedHash);

		checkf(
			ExpectedStructMetadata,
			TEXT("Shader is requesting a uniform buffer at slot %s with hash '%u', but a reverse lookup of the hash can't find it. The shader cache may be out of date."),
			*SlotRegistry.GetDebugDescription(Slot), ExpectedHash);

		const EUniformBufferBindingFlags BindingFlags = ExpectedStructMetadata->GetBindingFlags();

		checkf(EnumHasAnyFlags(BindingFlags, EUniformBufferBindingFlags::Static),
			TEXT("Shader requested a global uniform buffer of type '%s' at static slot '%s', but it is not registered with the Global binding flag. The shader cache may be out of date."),
			ExpectedStructMetadata->GetShaderVariableName(), *SlotRegistry.GetDebugDescription(Slot));

		// Structs can be bound both globally or per-shader, effectively leaving it up to the user to choose which to bind.
		// But that also means we can't validate existence at the global level.
		if (!EnumHasAnyFlags(BindingFlags, EUniformBufferBindingFlags::Shader))
		{
			UE_LOGF(LogRHICore, Fatal,
				"Shader requested a global uniform buffer of type '%ls' at static slot '%ls', but it was null. The uniform buffer should " "be bound using RHICmdList.SetStaticUniformBuffers() or passed into an RDG pass using SHADER_PARAMETER_STRUCT_REF() or " "SHADER_PARAMETER_RDG_UNIFORM_BUFFER().",
				ExpectedStructMetadata->GetShaderVariableName(), *SlotRegistry.GetDebugDescription(Slot));
		}
	}
	else
	{
		const FRHIUniformBufferLayout& Layout = UniformBuffer->GetLayout();

		if (Layout.GetHash() != ExpectedHash)
		{
			const FShaderParametersMetadata* ExpectedStructMetadata = FindUniformBufferStructByLayoutHash(ExpectedHash);

			checkf(
				ExpectedStructMetadata,
				TEXT("Shader is requesting uniform buffer '%s' at slot %s with hash '%u', but a reverse lookup of the hash can't find it. The shader cache may be out of date."),
				*Layout.GetDebugName(), *SlotRegistry.GetDebugDescription(Slot), ExpectedHash);

			checkf(
				false,
				TEXT("Shader attempted to bind uniform buffer '%s' at slot %s with hash '%u', but the shader expected '%s' with hash '%u'."),
				*Layout.GetDebugName(), *SlotRegistry.GetDebugDescription(Slot), Layout.GetHash(), ExpectedStructMetadata->GetShaderVariableName(), ExpectedHash);
		}
	}
#endif
}

void UE::RHICore::SetupShaderCodeValidationData(FRHIShader* RHIShader, FShaderCodeReader& ShaderCodeReader)
{
#if RHI_INCLUDE_SHADER_DEBUG_DATA && ENABLE_RHI_VALIDATION
	if (GRHIValidationEnabled && RHIShader)
	{
		int32 ShaderCodeValidationExtensionSize = 0;
		const uint8* ShaderCodeValidationExtensionData = ShaderCodeReader.FindOptionalDataAndSize(FShaderCodeValidationExtension::Key, ShaderCodeValidationExtensionSize);
		if (ShaderCodeValidationExtensionData && ShaderCodeValidationExtensionSize > 0)
		{
			FBufferReader ArValidationExtensionData((void*)ShaderCodeValidationExtensionData, ShaderCodeValidationExtensionSize, false);
			FShaderCodeValidationExtension ShaderCodeValidationExtension;
			ArValidationExtensionData << ShaderCodeValidationExtension;
			RHIShader->DebugStrideValidationData.Append(ShaderCodeValidationExtension.ShaderCodeValidationStride);
			RHIShader->DebugSRVTypeValidationData.Append(ShaderCodeValidationExtension.ShaderCodeValidationSRVType);
			RHIShader->DebugUAVTypeValidationData.Append(ShaderCodeValidationExtension.ShaderCodeValidationUAVType);
			RHIShader->DebugUBSizeValidationData.Append(ShaderCodeValidationExtension.ShaderCodeValidationUBSize);
		}
	}
#endif
}

void UE::RHICore::DispatchShaderBundleEmulation(
	FRHIComputeCommandList& InRHICmdList,
	FRHIShaderBundle* ShaderBundle,
	FRHIBuffer* ArgumentBuffer,
	TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
	TConstArrayView<FRHIShaderBundleComputeDispatch> Dispatches)
{
	for (const FRHIShaderBundleComputeDispatch& Dispatch : Dispatches)
	{
		if (!Dispatch.IsValid() || Dispatch.Shader == nullptr)
		{
			continue;
		}

		checkf(Dispatch.Shader->HasShaderBundleUsage(), TEXT("All shaders in a bundle must specify CFLAG_ShaderBundle"));

		SetComputePipelineState(InRHICmdList, Dispatch.Shader);

		if (SharedBindlessParameters.Num())
		{
			InRHICmdList.SetShaderParameters(
				Dispatch.Shader,
				{},
				{},
				{},
				SharedBindlessParameters
			);
		}

		if (Dispatch.Parameters->HasParameters())
		{
			InRHICmdList.SetShaderParameters(
				Dispatch.Shader,
				Dispatch.Parameters->ParametersData,
				Dispatch.Parameters->Parameters,
				Dispatch.Parameters->ResourceParameters,
				Dispatch.Parameters->BindlessParameters
			);
		}

		if (GRHISupportsShaderRootConstants)
		{
			InRHICmdList.SetShaderRootConstants(Dispatch.Constants);
		}

		const uint32 IndirectOffset = (Dispatch.RecordIndex * ShaderBundle->ArgStride) + ShaderBundle->ArgOffset;
		InRHICmdList.DispatchIndirectComputeShader(ArgumentBuffer, IndirectOffset);
	}
}

void UE::RHICore::DispatchShaderBundleEmulation(
	FRHICommandList& InRHICmdList,
	FRHIShaderBundle* ShaderBundle,
	FRHIBuffer* ArgumentBuffer,
	const FRHIShaderBundleGraphicsState& BundleState,
	TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
	TConstArrayView<FRHIShaderBundleGraphicsDispatch> Dispatches)
{
	if (Dispatches.Num() == 0)
	{
		return;
	}

	InRHICmdList.SetViewport(
		BundleState.ViewRect.Min.X,
		BundleState.ViewRect.Min.Y,
		BundleState.DepthMin,
		FMath::Min(BundleState.ViewRect.Max.X, 32767),
		FMath::Min(BundleState.ViewRect.Max.Y, 32767),
		BundleState.DepthMax
	);

	for (const FRHIShaderBundleGraphicsDispatch& Dispatch : Dispatches)
	{
		if (!Dispatch.IsValid())
		{
			continue;
		}

		const FBoundShaderStateInput& ShaderState = Dispatch.PipelineInitializer.BoundShaderState;
		FRHIGraphicsShader* MSVSShader = ShaderBundle->Mode == ERHIShaderBundleMode::MSPS ? (FRHIGraphicsShader*)ShaderState.GetMeshShader() : (FRHIGraphicsShader*)ShaderState.GetVertexShader();

		if (ShaderState.GetPixelShader() == nullptr || MSVSShader == nullptr)
		{
			continue;
		}

		checkf(ShaderState.GetPixelShader()->HasShaderBundleUsage(), TEXT("All shaders in a bundle must specify CFLAG_ShaderBundle"));
		checkf(MSVSShader->HasShaderBundleUsage(), TEXT("All shaders in a bundle must specify CFLAG_ShaderBundle"));

		SetGraphicsPipelineState(InRHICmdList, Dispatch.PipelineInitializer, BundleState.StencilRef);

		if (SharedBindlessParameters.Num())
		{
			InRHICmdList.SetShaderParameters(
				MSVSShader,
				{},
				{},
				{},
				SharedBindlessParameters
			);
		}

		if (Dispatch.Parameters_MSVS->HasParameters())
		{
			InRHICmdList.SetShaderParameters(
				MSVSShader,
				Dispatch.Parameters_MSVS->ParametersData,
				Dispatch.Parameters_MSVS->Parameters,
				Dispatch.Parameters_MSVS->ResourceParameters,
				Dispatch.Parameters_MSVS->BindlessParameters
			);
		}

		if (SharedBindlessParameters.Num())
		{
			InRHICmdList.SetShaderParameters(
				ShaderState.GetPixelShader(),
				{},
				{},
				{},
				SharedBindlessParameters
			);
		}

		if (Dispatch.Parameters_PS->HasParameters())
		{
			InRHICmdList.SetShaderParameters(
				ShaderState.GetPixelShader(),
				Dispatch.Parameters_PS->ParametersData,
				Dispatch.Parameters_PS->Parameters,
				Dispatch.Parameters_PS->ResourceParameters,
				Dispatch.Parameters_PS->BindlessParameters
			);
		}

		if (GRHISupportsShaderRootConstants)
		{
			InRHICmdList.SetShaderRootConstants(Dispatch.Constants);
		}

		const uint32 IndirectOffset = (Dispatch.RecordIndex * ShaderBundle->ArgStride) + ShaderBundle->ArgOffset;

		if (ShaderBundle->Mode == ERHIShaderBundleMode::MSPS)
		{
			InRHICmdList.DispatchIndirectMeshShader(ArgumentBuffer, IndirectOffset);
		}
		else
		{
			InRHICmdList.DrawPrimitiveIndirect(ArgumentBuffer, IndirectOffset);
		}
	}
}

const bool GRHIShaderDiagnosticEnabled = true;
void UE::RHICore::SetupShaderDiagnosticData(FRHIShader* RHIShader, FShaderCodeReader& ShaderCodeReader)
{
	if (RHIShader && GRHIShaderDiagnosticEnabled)
	{
		int32 ShaderDiagnosticExtensionSize = 0;
		const uint8* ShaderDiagnosticExtensionData = ShaderCodeReader.FindOptionalDataAndSize(FShaderDiagnosticExtension::Key, ShaderDiagnosticExtensionSize);
		if (ShaderDiagnosticExtensionData && ShaderDiagnosticExtensionSize > 0)
		{
			FBufferReader ArValidationExtensionData((void*)ShaderDiagnosticExtensionData, ShaderDiagnosticExtensionSize, false);
			FShaderDiagnosticExtension ShaderDiagnosticExtension;
			ArValidationExtensionData << ShaderDiagnosticExtension;
			RegisterDiagnosticMessages(ShaderDiagnosticExtension.ShaderDiagnosticDatas);
		}
	}
}

FCriticalSection GShaderDiagnosticDatasCS;
TArray<FShaderDiagnosticData> GShaderDiagnosticDatas;
void UE::RHICore::RegisterDiagnosticMessages(const TArray<FShaderDiagnosticData>& In)
{
	FScopeLock ScopeLock(&GShaderDiagnosticDatasCS);
	GShaderDiagnosticDatas.Append(In);
}

const FString* UE::RHICore::GetDiagnosticMessage(uint32 MessageID)
{
	FScopeLock ScopeLock(&GShaderDiagnosticDatasCS);
	if (const FShaderDiagnosticData* Found = GShaderDiagnosticDatas.FindByPredicate([MessageID](const FShaderDiagnosticData& In) { return In.Hash == MessageID; }))
	{
		return &Found->Message;
	}
	return nullptr;
}
