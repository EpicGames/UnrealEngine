// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHICoreStats.h"
#include "HAL/PlatformAtomics.h"
#include "RHIGlobals.h"
#include "RHICore.h"
#include "RHIResources.h"
#include "RHIStats.h"
#include "HAL/IConsoleManager.h"

static TAutoConsoleVariable<int32> CVarRHIReservedResourcesVirtualSizeWarningGB(
	TEXT("rhi.ReservedResources.VirtualSizeWarningGB"),
	256,
	TEXT("Report a warning if total allocated virtual size of all reserved resources exceeds this budget.\n")
	TEXT("Some platforms have a limited virtual address space for reserved resources, so it may be useful to know if an application is getting close to it."),
	ECVF_Default
);

namespace UE::RHICore
{


static void UpdateReservedResourceVirtualSizeStats(int64 SizeDelta)
{
	int64 PreviousSize = FPlatformAtomics::InterlockedAdd((volatile int64*)&GRHIGlobals.ReservedResources.VirtualSize, SizeDelta);

	if (SizeDelta > 0)
	{
		const uint64 BudgetSizeGB = uint64(CVarRHIReservedResourcesVirtualSizeWarningGB.GetValueOnAnyThread());
		const uint64 CurrentSizeGB = (uint64(PreviousSize) + uint64(SizeDelta)) >> 30;

		static bool bWarningReported = false;
		if (CurrentSizeGB > BudgetSizeGB && !bWarningReported)
		{
			UE_LOGF(LogRHICore, Warning,
				"Total reserved resource allocated virtual size (%d GB) exceeds the budget (%d GB). " "Use `rhi.DumpMemory` and `rhi.DumpResourceMemory` console commands to investigate.",
				uint32(CurrentSizeGB), uint32(BudgetSizeGB));

			bWarningReported = true;
		}
	}
}

void UpdateGlobalTextureStats(ETextureCreateFlags TextureFlags, ETextureDimension Dimension, uint64 TextureSizeInBytes, bool bOnlyStreamableTexturesInTexturePool, bool bAllocating)
{
	constexpr ETextureCreateFlags AllRenderTargetFlags = ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ResolveTargetable | ETextureCreateFlags::DepthStencilTargetable;

	const int64 TextureSizeDeltaInBytes = bAllocating ? static_cast<int64>(TextureSizeInBytes) : -static_cast<int64>(TextureSizeInBytes);

#if STATS
	if (EnumHasAnyFlags(TextureFlags, ETextureCreateFlags::ReservedResource))
	{
		INC_MEMORY_STAT_BY(STAT_ReservedUncommittedTextureMemory, TextureSizeDeltaInBytes);
	}
	else if (EnumHasAnyFlags(TextureFlags, AllRenderTargetFlags))
	{
		switch (Dimension)
		{
		default:
			checkNoEntry();
			[[fallthrough]];
		case ETextureDimension::Texture2D:
			[[fallthrough]];
		case ETextureDimension::Texture2DArray:
			INC_MEMORY_STAT_BY(STAT_RenderTargetMemory2D, TextureSizeDeltaInBytes);
			break;
		case ETextureDimension::Texture3D:
			INC_MEMORY_STAT_BY(STAT_RenderTargetMemory3D, TextureSizeDeltaInBytes);
			break;
		case ETextureDimension::TextureCube:
			[[fallthrough]];
		case ETextureDimension::TextureCubeArray:
			INC_MEMORY_STAT_BY(STAT_RenderTargetMemoryCube, TextureSizeDeltaInBytes);
			break;
		};
	}
	else if (EnumHasAnyFlags(TextureFlags, ETextureCreateFlags::UAV))
	{
		INC_MEMORY_STAT_BY(STAT_UAVTextureMemory, TextureSizeDeltaInBytes);
	}
	else
	{
		switch (Dimension)
		{
		default:
			checkNoEntry();
			[[fallthrough]];
		case ETextureDimension::Texture2D:
			[[fallthrough]];
		case ETextureDimension::Texture2DArray:
			INC_MEMORY_STAT_BY(STAT_TextureMemory2D, TextureSizeDeltaInBytes);
			break;
		case ETextureDimension::Texture3D:
			INC_MEMORY_STAT_BY(STAT_TextureMemory3D, TextureSizeDeltaInBytes);
			break;
		case ETextureDimension::TextureCube:
			[[fallthrough]];
		case ETextureDimension::TextureCubeArray:
			INC_MEMORY_STAT_BY(STAT_TextureMemoryCube, TextureSizeDeltaInBytes);
			break;
		};
	}
#endif
	const int64 AlignedTextureSizeInKB = static_cast<int64>(Align(TextureSizeInBytes, 1024) / 1024);
	const int64 TextureSizeDeltaInKB = bAllocating ? AlignedTextureSizeInKB : -AlignedTextureSizeInKB;

	const bool bAlwaysExcludedFromStreamingSize = EnumHasAnyFlags(TextureFlags,
		AllRenderTargetFlags
		| ETextureCreateFlags::UAV
		| ETextureCreateFlags::ForceIntoNonStreamingMemoryTracking
	);

	const bool bStreamable = EnumHasAnyFlags(TextureFlags, ETextureCreateFlags::Streamable);

	if (!bAlwaysExcludedFromStreamingSize && (!bOnlyStreamableTexturesInTexturePool || bStreamable))
	{
		FPlatformAtomics::InterlockedAdd((volatile int64*)&GRHIGlobals.StreamingTextureMemorySizeInKB, TextureSizeDeltaInKB);
	}
	else
	{
		FPlatformAtomics::InterlockedAdd((volatile int64*)&GRHIGlobals.NonStreamingTextureMemorySizeInKB, TextureSizeDeltaInKB);
	}

	if (EnumHasAnyFlags(TextureFlags, ETextureCreateFlags::ReservedResource))
	{
		UpdateReservedResourceVirtualSizeStats(TextureSizeDeltaInBytes);
	}
}

void UpdateGlobalTextureStats(const FRHITextureDesc& TextureDesc, uint64 TextureSizeInBytes, bool bOnlyStreamableTexturesInTexturePool, bool bAllocating)
{
	return UpdateGlobalTextureStats(TextureDesc.Flags, TextureDesc.Dimension, TextureSizeInBytes, bOnlyStreamableTexturesInTexturePool, bAllocating);
}

void FillBaselineTextureMemoryStats(FTextureMemoryStats& OutStats)
{
	OutStats.StreamingMemorySize    = GRHIGlobals.StreamingTextureMemorySizeInKB * 1024;
	OutStats.NonStreamingMemorySize = GRHIGlobals.NonStreamingTextureMemorySizeInKB * 1024;
	OutStats.TexturePoolSize        = GRHIGlobals.TexturePoolSize;
}

void UpdateGlobalBufferStats(const FRHIBufferDesc& BufferDesc, int64 BufferSizeDelta)
{
#if STATS
	if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::ReservedResource))
	{
		INC_MEMORY_STAT_BY(STAT_ReservedUncommittedBufferMemory, BufferSizeDelta);
	}
	else if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::VertexBuffer))
	{
		INC_MEMORY_STAT_BY(STAT_VertexBufferMemory, BufferSizeDelta);
	}
	else if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::IndexBuffer))
	{
		INC_MEMORY_STAT_BY(STAT_IndexBufferMemory, BufferSizeDelta);
	}
	else if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::AccelerationStructure))
	{
		INC_MEMORY_STAT_BY(STAT_RTAccelerationStructureMemory, BufferSizeDelta);
	}
	else if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::ByteAddressBuffer))
	{
		INC_MEMORY_STAT_BY(STAT_ByteAddressBufferMemory, BufferSizeDelta);
	}
	else if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::DrawIndirect))
	{
		INC_MEMORY_STAT_BY(STAT_DrawIndirectBufferMemory, BufferSizeDelta);
	}
	else if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::StructuredBuffer))
	{
		INC_MEMORY_STAT_BY(STAT_StructuredBufferMemory, BufferSizeDelta);
	}
	else
	{
		INC_MEMORY_STAT_BY(STAT_MiscBufferMemory, BufferSizeDelta);
	}
#endif

	FPlatformAtomics::InterlockedAdd((volatile int64*)&GRHIGlobals.BufferMemorySize, BufferSizeDelta);

	if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::ReservedResource))
	{
		UpdateReservedResourceVirtualSizeStats(BufferSizeDelta);
	}
}

void UpdateGlobalUniformBufferStats(int64 BufferSize, bool bAllocating)
{
	const int64 BufferSizeDelta = bAllocating ? static_cast<int64>(BufferSize) : -static_cast<int64>(BufferSize);

	INC_MEMORY_STAT_BY(STAT_UniformBufferMemory, BufferSizeDelta);
	FPlatformAtomics::InterlockedAdd((volatile int64*)&GRHIGlobals.UniformBufferMemorySize, BufferSizeDelta);
}

void UpdateReservedResourceStatsOnCommit(int64 CommitDelta, bool bBuffer, bool bCommitting)
{
#if STATS
	const int64 CommittedDelta = bCommitting ? static_cast<int64>(CommitDelta) : -static_cast<int64>(CommitDelta);

	if (bBuffer)
	{
		INC_MEMORY_STAT_BY(STAT_ReservedCommittedBufferMemory, CommittedDelta);
		DEC_MEMORY_STAT_BY(STAT_ReservedUncommittedBufferMemory, CommittedDelta);
	}
	else
	{
		INC_MEMORY_STAT_BY(STAT_ReservedCommittedTextureMemory, CommittedDelta);
		DEC_MEMORY_STAT_BY(STAT_ReservedUncommittedTextureMemory, CommittedDelta);
	}
#endif
}

} // namespace UE::RHICore
