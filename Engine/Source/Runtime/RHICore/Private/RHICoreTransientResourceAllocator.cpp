// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHICoreTransientResourceAllocator.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "RHICommandList.h"
#include "RHICoreNvidiaAftermath.h"
#include "RHICore.h"

static bool GRHITransientAllocatorParallelResourceCreation = true;
static FAutoConsoleVariableRef CVarRHITransientAllocatorParallelResourceCreation(
	TEXT("RHI.TransientAllocator.ParallelResourceCreation"),
	GRHITransientAllocatorParallelResourceCreation,
	TEXT("If enabled, a task is launched for each placed resource that is created."),
	ECVF_RenderThreadSafe);

static int32 GRHITransientAllocatorMinimumHeapSize = 128;
static FAutoConsoleVariableRef CVarRHITransientAllocatorMinimumHeapSize(
	TEXT("RHI.TransientAllocator.MinimumHeapSize"),
	GRHITransientAllocatorMinimumHeapSize,
	TEXT("Minimum size of an RHI transient heap in MB. Heaps will default to this size and grow to the maximum based on the first allocation (Default 128)."),
	ECVF_ReadOnly);

static int32 GRHITransientAllocatorBufferCacheSize = 64;
static FAutoConsoleVariableRef CVarRHITransientAllocatorBufferCacheSize(
	TEXT("RHI.TransientAllocator.BufferCacheSize"),
	GRHITransientAllocatorBufferCacheSize,
	TEXT("The maximum number of RHI buffers to cache on each heap before garbage collecting."),
	ECVF_ReadOnly);

static int32 GRHITransientAllocatorTextureCacheSize = 64;
static FAutoConsoleVariableRef CVarRHITransientAllocatorTextureCacheSize(
	TEXT("RHI.TransientAllocator.TextureCacheSize"),
	GRHITransientAllocatorTextureCacheSize,
	TEXT("The maximum number of RHI textures to cache on each heap before garbage collecting."),
	ECVF_ReadOnly);

static int32 GRHITransientAllocatorGarbageCollectLatency = 16;
static FAutoConsoleVariableRef CVarRHITransientAllocatorGarbageCollectLatency(
	TEXT("RHI.TransientAllocator.GarbageCollectLatency"),
	GRHITransientAllocatorGarbageCollectLatency,
	TEXT("Amount of update cycles before memory is reclaimed."),
	ECVF_ReadOnly);

TRACE_DECLARE_INT_COUNTER(TransientResourceCreateCount, TEXT("TransientAllocator/ResourceCreateCount"));

TRACE_DECLARE_INT_COUNTER(TransientTextureCreateCount, TEXT("TransientAllocator/TextureCreateCount"));
TRACE_DECLARE_INT_COUNTER(TransientTextureCount, TEXT("TransientAllocator/TextureCount"));
TRACE_DECLARE_INT_COUNTER(TransientTextureCacheSize, TEXT("TransientAllocator/TextureCacheSize"));
TRACE_DECLARE_FLOAT_COUNTER(TransientTextureCacheHitPercentage, TEXT("TransientAllocator/TextureCacheHitPercentage"));

TRACE_DECLARE_INT_COUNTER(TransientBufferCreateCount, TEXT("TransientAllocator/BufferCreateCount"));
TRACE_DECLARE_INT_COUNTER(TransientBufferCount, TEXT("TransientAllocator/BufferCount"));
TRACE_DECLARE_INT_COUNTER(TransientBufferCacheSize, TEXT("TransientAllocator/BufferCacheSize"));
TRACE_DECLARE_FLOAT_COUNTER(TransientBufferCacheHitPercentage, TEXT("TransientAllocator/BufferCacheHitPercentage"));

TRACE_DECLARE_INT_COUNTER(TransientPageMapCount, TEXT("TransientAllocator/PageMapCount"));
TRACE_DECLARE_INT_COUNTER(TransientPageAllocateCount, TEXT("TransientAllocator/PageAllocateCount"));
TRACE_DECLARE_INT_COUNTER(TransientPageSpanCount, TEXT("TransientAllocator/PageSpanCount"));

TRACE_DECLARE_INT_COUNTER(TransientMemoryRangeCount, TEXT("TransientAllocator/MemoryRangeCount"));

TRACE_DECLARE_MEMORY_COUNTER(TransientMemoryUsed, TEXT("TransientAllocator/MemoryUsed"));
TRACE_DECLARE_MEMORY_COUNTER(TransientMemoryRequested, TEXT("TransientAllocator/MemoryRequested"));

DECLARE_STATS_GROUP(TEXT("RHI: Transient Memory"), STATGROUP_RHITransientMemory, STATCAT_Advanced);

DECLARE_MEMORY_STAT(TEXT("Memory Used"), STAT_RHITransientMemoryUsed, STATGROUP_RHITransientMemory);
DECLARE_MEMORY_STAT(TEXT("Memory Aliased"), STAT_RHITransientMemoryAliased, STATGROUP_RHITransientMemory);
DECLARE_MEMORY_STAT(TEXT("Memory Requested"), STAT_RHITransientMemoryRequested, STATGROUP_RHITransientMemory);
DECLARE_MEMORY_STAT(TEXT("Buffer Memory Requested"), STAT_RHITransientBufferMemoryRequested, STATGROUP_RHITransientMemory);
DECLARE_MEMORY_STAT(TEXT("Texture Memory Requested"), STAT_RHITransientTextureMemoryRequested, STATGROUP_RHITransientMemory);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Resources"), STAT_RHITransientResources, STATGROUP_RHITransientMemory);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Textures"), STAT_RHITransientTextures, STATGROUP_RHITransientMemory);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Buffers"), STAT_RHITransientBuffers, STATGROUP_RHITransientMemory);

DECLARE_LLM_MEMORY_STAT(TEXT("RHI Transient Resources"), STAT_RHITransientResourcesLLM, STATGROUP_LLMFULL);

#if ENABLE_LOW_LEVEL_MEM_TRACKER // if LLM_DEFINE_TAG is something we need to dll export it
RHICORE_API
#endif
LLM_DEFINE_TAG(RHITransientResources, NAME_None, NAME_None, GET_STATFNAME(STAT_RHITransientResourcesLLM), GET_STATFNAME(STAT_EngineSummaryLLM));

//////////////////////////////////////////////////////////////////////////

void FRHITransientMemoryStats::Submit(uint64 UsedSize)
{
	const int32 CreateResourceCount = Textures.CreateCount + Buffers.CreateCount;
	const int64 MemoryUsed = UsedSize;
	const int64 MemoryRequested = AliasedSize;
	const float ToMB = 1.0f / (1024.0f * 1024.0f);

	TRACE_COUNTER_SET(TransientResourceCreateCount, CreateResourceCount);
	TRACE_COUNTER_SET(TransientTextureCreateCount, Textures.CreateCount);
	TRACE_COUNTER_SET(TransientBufferCreateCount, Buffers.CreateCount);
	TRACE_COUNTER_SET(TransientMemoryUsed, MemoryUsed);
	TRACE_COUNTER_SET(TransientMemoryRequested, MemoryRequested);

	CSV_CUSTOM_STAT_GLOBAL(TransientResourceCreateCount, CreateResourceCount, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_GLOBAL(TransientMemoryUsedMB, static_cast<float>(MemoryUsed * ToMB) , ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_GLOBAL(TransientMemoryAliasedMB, static_cast<float>(MemoryRequested * ToMB), ECsvCustomStatOp::Set);

	SET_MEMORY_STAT(STAT_RHITransientMemoryUsed, UsedSize);
	SET_MEMORY_STAT(STAT_RHITransientMemoryAliased, AliasedSize);
	SET_MEMORY_STAT(STAT_RHITransientMemoryRequested, Textures.AllocatedSize + Buffers.AllocatedSize);
	SET_MEMORY_STAT(STAT_RHITransientBufferMemoryRequested, Buffers.AllocatedSize);
	SET_MEMORY_STAT(STAT_RHITransientTextureMemoryRequested, Textures.AllocatedSize);

	SET_DWORD_STAT(STAT_RHITransientTextures, Textures.AllocationCount);
	SET_DWORD_STAT(STAT_RHITransientBuffers, Buffers.AllocationCount);
	SET_DWORD_STAT(STAT_RHITransientResources, Textures.AllocationCount + Buffers.AllocationCount);

	Reset();
}

//////////////////////////////////////////////////////////////////////////
// Transient Resource Heap Allocator
//////////////////////////////////////////////////////////////////////////

FRHITransientHeapAllocator::FRHITransientHeapAllocator(uint64 InCapacity, uint32 InAlignment)
	: Capacity(InCapacity)
	, AlignmentMin(InAlignment)
{
	HeadHandle = CreateRange();
	InsertRange(HeadHandle, nullptr, {}, 0, Capacity);
}

FRHITransientHeapAllocation FRHITransientHeapAllocator::Allocate(const FRHITransientAllocationFences& Fences, uint64 Size, uint32 Alignment, TArray<FAliasingOverlap>& OutAliasingOverlaps)
{
	check(Size > 0);

	if (Alignment < AlignmentMin)
	{
		Alignment = AlignmentMin;
	}

	TArray<FRangeHandle, TInlineAllocator<64>> RangeCandidates;

	FRangeHandle Handle = GetFirstFreeRangeHandle();
	FRangeHandle FirstPreviousHandle = HeadHandle;
	FRangeHandle PreviousHandle = InvalidRangeHandle;
	uint64 FirstAllocationRegionMin = 0;
	uint64 AllocationMin = 0;
	uint64 AllocationMax = 0;
	uint64 NextRangeMin  = 0;
	uint64 LeftoverSize  = 0;
	bool bAllocationComplete = false;

	while (Handle != InvalidRangeHandle)
	{
		FRange& Range = Ranges[Handle];
		const uint64 RangeMax = Range.Offset + Range.Size;

		const auto NextRegion = [&]
		{
			PreviousHandle = Handle;
			Handle = Range.NextFreeHandle;
		};

		// Specify the initial min / max bounds based off the current candidate range.
		if (RangeCandidates.IsEmpty())
		{
			const uint64 AlignedOffset = Align(GpuVirtualAddress + Range.Offset, Alignment) - GpuVirtualAddress;

			// Skip regions smaller than the alignment padding.
			if (AlignedOffset >= RangeMax)
			{
				FirstPreviousHandle = InvalidRangeHandle;
				NextRegion();
				continue;
			}

			FirstAllocationRegionMin = NextRangeMin = Range.Offset;
			AllocationMin = AlignedOffset;
			AllocationMax = AlignedOffset + Size;
		}

		// Range is allowed to be part of this allocation.
		if (Range.Offset == NextRangeMin)
		{
			ON_SCOPE_EXIT { NextRegion(); };

			if (!FRHITransientAllocationFences::Contains(Range.Fences, Fences))
			{
				if (FirstPreviousHandle == InvalidRangeHandle)
				{
					FirstPreviousHandle = PreviousHandle;
				}

				RangeCandidates.Emplace(Handle);

				// Range is large enough to service remaining allocation
				if (AllocationMax <= RangeMax)
				{
					LeftoverSize = RangeMax - AllocationMax;
					bAllocationComplete = true;
					break;
				}

				NextRangeMin += Range.Size;
				continue;
			}
		}

		RangeCandidates.Reset();
		FirstPreviousHandle = InvalidRangeHandle;
	}

	FRHITransientHeapAllocation Allocation;

	if (bAllocationComplete)
	{
		check(!RangeCandidates.IsEmpty())
		const uint64 AlignedSize  = AllocationMax - FirstAllocationRegionMin;
		const uint64 AlignmentPad = AlignedSize - Size;

		AllocationCount++;
		UsedSize       += AlignedSize;
		AlignmentWaste += AlignmentPad;

		for (int32 Index = 0; Index < RangeCandidates.Num(); ++Index)
		{
			int32 RangeIndex = RangeCandidates[Index];
			const FRange& Range = Ranges[RangeIndex];

			if (FRHITransientResource* ResourceToOverlap = Range.Resource)
			{
				OutAliasingOverlaps.Emplace(ResourceToOverlap, FRHITransientAllocationFences::GetAcquireFence(Range.Fences, Fences));
			}

			if (Index < RangeCandidates.Num() - 1)
			{
				RemoveRange(FirstPreviousHandle, RangeIndex);
			}
		}
	
		if (LeftoverSize > 0)
		{
			FRange& LastRange = Ranges[RangeCandidates.Last()];
			LastRange.Offset  = AllocationMax;
			LastRange.Size    = LeftoverSize;
		}
		else
		{
			RemoveRange(FirstPreviousHandle, RangeCandidates.Last());
		}
		
		Allocation.Size   = Size;
		Allocation.Offset = AllocationMin;
		Allocation.AlignmentPad = AlignmentPad;
	}

	Validate();
	return Allocation;
}

void FRHITransientHeapAllocator::Deallocate(FRHITransientResource* Resource, const FRHITransientAllocationFences& Fences)
{
	check(Resource);
	const FRHITransientHeapAllocation& Allocation = Resource->GetHeapAllocation();
	check(Allocation.Size > 0 && Allocation.Size <= UsedSize);

	// Reconstruct the original range offset by subtracting the alignment pad, and expand the size accordingly.
	const uint64 RangeToFreeOffset = Allocation.Offset - Allocation.AlignmentPad;
	const uint64 RangeToFreeSize = Allocation.Size + Allocation.AlignmentPad;
	const uint64 RangeToFreeEnd = RangeToFreeOffset + RangeToFreeSize;

	FRangeHandle PreviousHandle = HeadHandle;
	FRangeHandle Handle = GetFirstFreeRangeHandle();

	while (Handle != InvalidRangeHandle)
	{
		const FRange& Range = Ranges[Handle];

		// Find the first free range after the one being freed.
		if (RangeToFreeOffset < Range.Offset)
		{
			break;
		}

		PreviousHandle = Handle;
		Handle = Range.NextFreeHandle;
	}

	InsertRange(PreviousHandle, Resource, Fences, RangeToFreeOffset, RangeToFreeSize);

	UsedSize       -= RangeToFreeSize;
	AlignmentWaste -= Allocation.AlignmentPad;
	AllocationCount--;

	Validate();
}

void FRHITransientHeapAllocator::Flush()
{
	FRangeHandle Handle = GetFirstFreeRangeHandle();
	FRangeHandle PreviousHandle = InvalidRangeHandle;

	while (Handle != InvalidRangeHandle)
	{
		FRange& Range = Ranges[Handle];
		Range.Fences = {};
		Range.Resource = nullptr;

		if (PreviousHandle != InvalidRangeHandle)
		{
			FRange& PreviousRange = Ranges[PreviousHandle];

			if (PreviousRange.Offset + PreviousRange.Size == Range.Offset)
			{
				PreviousRange.Size += Range.Size;
				Handle = RemoveRange(PreviousHandle, Handle);
				continue;
			}
		}

		Handle = Range.NextFreeHandle;
		PreviousHandle = Handle;
	}
}

void FRHITransientHeapAllocator::Validate()
{
#if UE_BUILD_DEBUG
	uint64 DerivedFreeSize = 0;

	FRangeHandle PreviousHandle = HeadHandle;
	FRangeHandle NextHandle = InvalidRangeHandle;
	FRangeHandle Handle = GetFirstFreeRangeHandle();

	while (Handle != InvalidRangeHandle)
	{
		const FRange& Range = Ranges[Handle];
		DerivedFreeSize += Range.Size;

		if (PreviousHandle != HeadHandle)
		{
			const FRange& PreviousRange = Ranges[PreviousHandle];

			// Checks that the ranges are sorted.
			check(PreviousRange.Offset + PreviousRange.Size <= Range.Offset);
		}

		PreviousHandle = Handle;
		Handle = Range.NextFreeHandle;
	}

	check(Capacity == DerivedFreeSize + UsedSize);
#endif
}

//////////////////////////////////////////////////////////////////////////

FRHITransientTexture* FRHITransientHeap::CreateTexture(
	const FRHITextureCreateInfo& CreateInfo,
	const TCHAR* DebugName,
	const FRHITransientAllocationFences& Fences,
	uint64 CurrentAllocatorCycle,
	uint64 TextureSize,
	uint32 TextureAlignment,
	FCreateTextureFunction CreateTextureFunction)
{
	FRHITransientHeapAllocation Allocation = Allocator.Allocate(Fences, TextureSize, TextureAlignment, AliasingOverlaps);
	Allocation.Heap = this;

	if (!Allocation.IsValid())
	{
		return nullptr;
	}

	FRHITransientTexture* Texture = Textures.Acquire(ComputeHash(CreateInfo, Allocation.Offset), [&](uint64 Hash)
	{
		Stats.Textures.CreateCount++;
		return CreateTextureFunction(FResourceInitializer(Allocation, Hash));
	});

	check(Texture);
	Texture->Acquire(DebugName, Fences.GetSinglePipeline(), Fences.GetPipelines(), CurrentAllocatorCycle);
	AllocateMemoryInternal(Texture, Allocation);
	Stats.AllocateTexture(Allocation.Size);
	return Texture;
}

void FRHITransientHeap::DeallocateMemory(FRHITransientTexture* Texture, const FRHITransientAllocationFences& Fences)
{
	DeallocateMemoryInternal(Texture, Fences);
	Stats.DeallocateTexture(Texture->GetSize());
}

FRHITransientBuffer* FRHITransientHeap::CreateBuffer(
	const FRHIBufferCreateInfo& CreateInfo,
	const TCHAR* DebugName,
	const FRHITransientAllocationFences& Fences,
	uint64 CurrentAllocatorCycle,
	uint64 BufferSize,
	uint32 BufferAlignment,
	FCreateBufferFunction CreateBufferFunction)
{
	FRHITransientHeapAllocation Allocation = Allocator.Allocate(Fences, BufferSize, BufferAlignment, AliasingOverlaps);
	Allocation.Heap = this;

	if (!Allocation.IsValid())
	{
		return nullptr;
	}

	FRHITransientBuffer* Buffer = Buffers.Acquire(ComputeHash(CreateInfo, Allocation.Offset), [&](uint64 Hash)
	{
		Stats.Buffers.CreateCount++;
		return CreateBufferFunction(FResourceInitializer(Allocation, Hash));
	});

	check(Buffer);
	Buffer->Acquire(DebugName, Fences.GetSinglePipeline(), Fences.GetPipelines(), CurrentAllocatorCycle);
	AllocateMemoryInternal(Buffer, Allocation);
	Stats.AllocateBuffer(Allocation.Size);
	return Buffer;
}

void FRHITransientHeap::DeallocateMemory(FRHITransientBuffer* Buffer, const FRHITransientAllocationFences& Fences)
{
	DeallocateMemoryInternal(Buffer, Fences);
	Stats.DeallocateBuffer(Buffer->GetSize());
}

void FRHITransientHeap::AllocateMemoryInternal(FRHITransientResource* Resource, const FRHITransientHeapAllocation& Allocation)
{
	Resource->GetHeapAllocation() = Allocation;

	for (const FRHITransientHeapAllocator::FAliasingOverlap& AliasingOverlap : AliasingOverlaps)
	{
		Resource->AddAliasingOverlap(AliasingOverlap.Resource, AliasingOverlap.AcquireFence);
	}
	AliasingOverlaps.Reset();

	CommitSize = FMath::Max(CommitSize, Allocation.Offset + Allocation.Size);
}

void FRHITransientHeap::DeallocateMemoryInternal(FRHITransientResource* Resource, const FRHITransientAllocationFences& Fences)
{
	Resource->Discard(Fences);
	Allocator.Deallocate(Resource, Fences);
}

void FRHITransientHeap::Flush(uint64 AllocatorCycle, FRHITransientMemoryStats& OutMemoryStats, FRHITransientAllocationStats* OutAllocationStats)
{
	const bool bHasDeallocations = Stats.HasDeallocations();
	OutMemoryStats.Accumulate(Stats);
	Stats.Reset();

	Allocator.Flush();

	if (OutAllocationStats)
	{
		const auto AddResourceToStats = [](FRHITransientAllocationStats& AllocationStats, FRHITransientResource* Resource)
		{
			const FRHITransientHeapAllocation& HeapAllocation = Resource->GetHeapAllocation();

			FRHITransientAllocationStats::FAllocation Allocation;
			Allocation.OffsetMin = HeapAllocation.Offset;
			Allocation.OffsetMax = HeapAllocation.Offset + HeapAllocation.Size;
			Allocation.MemoryRangeIndex = AllocationStats.MemoryRanges.Num();

			AllocationStats.Resources.Emplace(Resource, FRHITransientAllocationStats::FAllocationArray{ Allocation });
		};

		OutAllocationStats->Resources.Reserve(OutAllocationStats->Resources.Num() + Textures.GetAllocatedCount() + Buffers.GetAllocatedCount());

		for (FRHITransientTexture* Texture : Textures.GetAllocated())
		{
			AddResourceToStats(*OutAllocationStats, Texture);
		}

		for (FRHITransientBuffer* Buffer : Buffers.GetAllocated())
		{
			AddResourceToStats(*OutAllocationStats, Buffer);
		}

		FRHITransientAllocationStats::FMemoryRange MemoryRange;
		MemoryRange.Capacity = GetCapacity();
		MemoryRange.CommitSize = GetCommitSize();
		OutAllocationStats->MemoryRanges.Add(MemoryRange);
	}

	// Atomically update the high watermark. CommitSizeMax is read/reset by GarbageCollect on the RHI thread.
	uint64 CurrentMax = CommitSizeMax.load(std::memory_order_relaxed);
	while (CommitSize > CurrentMax && !CommitSizeMax.compare_exchange_weak(CurrentMax, CommitSize, std::memory_order_relaxed)) {}

	if (bHasDeallocations)
	{
		CommitSize = 0;

		{
			Textures.Forfeit(GFrameCounterRenderThread);

			for (FRHITransientTexture* Texture : Textures.GetAllocated())
			{
				const FRHITransientHeapAllocation& Allocation = Texture->GetHeapAllocation();
				CommitSize = FMath::Max(CommitSize, Allocation.Offset + Allocation.Size);
			}
		}

		{
			Buffers.Forfeit(GFrameCounterRenderThread);

			for (FRHITransientBuffer* Buffer : Buffers.GetAllocated())
			{
				const FRHITransientHeapAllocation& Allocation = Buffer->GetHeapAllocation();
				CommitSize = FMath::Max(CommitSize, Allocation.Offset + Allocation.Size);
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////

FRHITransientHeapCache::FInitializer FRHITransientHeapCache::FInitializer::CreateDefault()
{
	FInitializer Initializer;
	Initializer.MinimumHeapSize = GRHITransientAllocatorMinimumHeapSize * 1024 * 1024;
	Initializer.HeapAlignment = 64 * 1024;
	Initializer.BufferCacheSize = GRHITransientAllocatorBufferCacheSize;
	Initializer.TextureCacheSize = GRHITransientAllocatorTextureCacheSize;
	Initializer.GarbageCollectLatency = GRHITransientAllocatorGarbageCollectLatency;
	return Initializer;
}

FRHITransientHeapCache::~FRHITransientHeapCache()
{
	for (FRHITransientHeap* Heap : LiveList)
	{
		delete Heap;
	}
	LiveList.Empty();
	FreeList.Empty();
}

FRHITransientHeap* FRHITransientHeapCache::Acquire(uint64 FirstAllocationSize, ERHITransientHeapFlags FirstAllocationHeapFlags)
{
	FScopeLock Lock(&CriticalSection);

	for (int32 HeapIndex = FreeList.Num() - 1; HeapIndex >= 0; --HeapIndex)
	{
		FRHITransientHeap* Heap = FreeList[HeapIndex];

		if (Heap->IsAllocationSupported(FirstAllocationSize, FirstAllocationHeapFlags))
		{
			FreeList.RemoveAt(HeapIndex);
			return Heap;
		}
	}

	FRHITransientHeap::FInitializer HeapInitializer;
	HeapInitializer.Size = GetHeapSize(FirstAllocationSize);
	HeapInitializer.Alignment = Initializer.HeapAlignment;
	if (GNumExplicitGPUsForRendering > 1)
	{
		// With multi-GPU, we need separate GPU0 only heaps for NNE accessible buffers.  Required by DirectML.  Note that the calling
		// code only sets one flag for a given allocation, so if the flag is NNE, create a heap with that flag alone, otherwise create
		// a heap with the rest of the flags (or if bSupportsAllHeapFlags is false, that also forces a single flag per heap).
		HeapInitializer.Flags = Initializer.bSupportsAllHeapFlags && FirstAllocationHeapFlags != ERHITransientHeapFlags::AllowNNEBuffers ?
			ERHITransientHeapFlags::AllowBuffers | ERHITransientHeapFlags::AllowTextures | ERHITransientHeapFlags::AllowRenderTargets : FirstAllocationHeapFlags;
	}
	else
	{
		HeapInitializer.Flags = Initializer.bSupportsAllHeapFlags ? ERHITransientHeapFlags::AllowAll : FirstAllocationHeapFlags;
	}
	HeapInitializer.TextureCacheSize = Initializer.TextureCacheSize;
	HeapInitializer.BufferCacheSize = Initializer.BufferCacheSize;

	LLM_SCOPE_BYTAG(RHITransientResources);
	FRHITransientHeap* Heap = CreateHeap(HeapInitializer);
	check(Heap);

	LiveList.Emplace(Heap);
	return Heap;
}

void FRHITransientHeapCache::Forfeit(TConstArrayView<FRHITransientHeap*> InForfeitedHeaps)
{
	FScopeLock Lock(&CriticalSection);

	LiveList.Reserve(InForfeitedHeaps.Num());
	for (int32 HeapIndex = InForfeitedHeaps.Num() - 1; HeapIndex >= 0; --HeapIndex)
	{
		FRHITransientHeap* Heap = InForfeitedHeaps[HeapIndex];
		check(Heap->IsEmpty());
		Heap->LastUsedGarbageCollectCycle = GarbageCollectCycle;
		FreeList.Add(Heap);
	}
}

void FRHITransientHeapCache::GarbageCollect()
{
	FScopeLock Lock(&CriticalSection);

	uint64 TotalCommitSize = 0;

	for (int32 HeapIndex = 0; HeapIndex < FreeList.Num(); ++HeapIndex)
	{
		FRHITransientHeap* Heap = FreeList[HeapIndex];

		if (Heap->GetLastUsedGarbageCollectCycle() + Initializer.GarbageCollectLatency <= GarbageCollectCycle)
		{
			FreeList.RemoveAt(HeapIndex);
			LiveList.Remove(Heap);
			HeapIndex--;

			delete Heap;
		}
	}
	
	for (FRHITransientHeap* Heap : LiveList)
	{
		TotalCommitSize += Initializer.bSupportsVirtualMapping ? Heap->CommitSizeMax.exchange(0, std::memory_order_relaxed) : Heap->GetCapacity();
	}

	TRACE_COUNTER_SET(TransientMemoryRangeCount, LiveList.Num());

	Stats.Submit(TotalCommitSize);

	GarbageCollectCycle++;
}

//////////////////////////////////////////////////////////////////////////

FRHITransientResourceHeapAllocator::FRHITransientResourceHeapAllocator(FRHITransientHeapCache& InHeapCache)
	: HeapCache(InHeapCache)
{}

FRHITransientResourceHeapAllocator::~FRHITransientResourceHeapAllocator() = default;

FRHITransientTexture* FRHITransientResourceHeapAllocator::CreateTextureInternal(
	const FRHITextureCreateInfo& CreateInfo,
	const TCHAR* DebugName,
	const FRHITransientAllocationFences& Fences,
	uint64 TextureSize,
	uint32 TextureAlignment,
	FRHITransientHeap::FCreateTextureFunction CreateTextureFunction)
{
	const ERHITransientHeapFlags TextureHeapFlags =
		EnumHasAnyFlags(CreateInfo.Flags, TexCreate_RenderTargetable | TexCreate_ResolveTargetable | TexCreate_DepthStencilTargetable | TexCreate_DepthStencilResolveTarget)
		? ERHITransientHeapFlags::AllowRenderTargets
		: ERHITransientHeapFlags::AllowTextures;

	FRHITransientTexture* Texture = nullptr;

	for (FRHITransientHeap* Heap : Heaps)
	{
		if (!Heap->IsAllocationSupported(TextureSize, TextureHeapFlags))
		{
			continue;
		}

		Texture = Heap->CreateTexture(CreateInfo, DebugName, Fences, CurrentCycle, TextureSize, TextureAlignment, CreateTextureFunction);

		if (Texture)
		{
			break;
		}
	}

	if (!Texture)
	{
		FRHITransientHeap* Heap = HeapCache.Acquire(TextureSize, TextureHeapFlags);
		Heaps.Emplace(Heap);

		Texture = Heap->CreateTexture(CreateInfo, DebugName, Fences, CurrentCycle, TextureSize, TextureAlignment, CreateTextureFunction);
	}

	if (!Texture)
	{
		UE_LOG(LogRHICore, Fatal,
			TEXT("Transient allocator failed to allocate Texture %s. Extent: (%u, %u), Depth: %u, ArraySize: %u, NumMips: %u. Allocation Size: %" UINT64_FMT ", Allocation Alignment %u."),
			DebugName, CreateInfo.Extent.X, CreateInfo.Extent.Y, CreateInfo.Depth, CreateInfo.ArraySize, CreateInfo.NumMips, TextureSize, TextureAlignment);
	}

	IF_RHICORE_TRANSIENT_ALLOCATOR_DEBUG(ActiveResources.Emplace(Texture));
	return Texture;
}

FRHITransientBuffer* FRHITransientResourceHeapAllocator::CreateBufferInternal(
	const FRHIBufferCreateInfo& CreateInfo,
	const TCHAR* DebugName,
	const FRHITransientAllocationFences& Fences,
	uint32 BufferSize,
	uint32 BufferAlignment,
	FRHITransientHeap::FCreateBufferFunction CreateBufferFunction)
{
	FRHITransientBuffer* Buffer = nullptr;

#if WITH_MGPU
	ERHITransientHeapFlags BufferHeapFlag = (GNumExplicitGPUsForRendering > 1) && EnumHasAnyFlags(CreateInfo.Usage, EBufferUsageFlags::NNE) ? ERHITransientHeapFlags::AllowNNEBuffers : ERHITransientHeapFlags::AllowBuffers;
#else
	ERHITransientHeapFlags BufferHeapFlag = ERHITransientHeapFlags::AllowBuffers;
#endif

	for (FRHITransientHeap* Heap : Heaps)
	{
		if (!Heap->IsAllocationSupported(BufferSize, BufferHeapFlag))
		{
			continue;
		}

		Buffer = Heap->CreateBuffer(CreateInfo, DebugName, Fences, CurrentCycle, BufferSize, BufferAlignment, CreateBufferFunction);

		if (Buffer)
		{
			break;
		}
	}

	if (!Buffer)
	{
		FRHITransientHeap* Heap = HeapCache.Acquire(BufferSize, BufferHeapFlag);
		Heaps.Emplace(Heap);

		Buffer = Heap->CreateBuffer(CreateInfo, DebugName, Fences, CurrentCycle, BufferSize, BufferAlignment, CreateBufferFunction);
	}

	if (!Buffer)
	{
		UE_LOGF(LogRHICore, Fatal,
			"Transient allocator failed to allocate Buffer %ls. Size: %u, Stride: %u. Allocation Size: %u, Allocation Alignment %u.",
			DebugName, CreateInfo.Size, CreateInfo.Stride, BufferSize, BufferAlignment);
	}

	IF_RHICORE_TRANSIENT_ALLOCATOR_DEBUG(ActiveResources.Emplace(Buffer));
	return Buffer;
}

void FRHITransientResourceHeapAllocator::DeallocateMemory(FRHITransientTexture* Texture, const FRHITransientAllocationFences& Fences)
{
	check(Texture);

	FRHITransientHeap* Heap = Texture->GetHeapAllocation().Heap;

	check(Heap);
	check(Heaps.Contains(Heap));

	Heap->DeallocateMemory(Texture, Fences);
	DeallocationCount++;

	IF_RHICORE_TRANSIENT_ALLOCATOR_DEBUG(ActiveResources.Remove(Texture));
}

void FRHITransientResourceHeapAllocator::DeallocateMemory(FRHITransientBuffer* Buffer, const FRHITransientAllocationFences& Fences)
{
	check(Buffer);

	FRHITransientHeap* Heap = Buffer->GetHeapAllocation().Heap;

	check(Heap);
	check(Heaps.Contains(Heap));

	Heap->DeallocateMemory(Buffer, Fences);
	DeallocationCount++;

	IF_RHICORE_TRANSIENT_ALLOCATOR_DEBUG(ActiveResources.Remove(Buffer));
}

void FRHITransientResourceHeapAllocator::SetCreateMode(ERHITransientResourceCreateMode InCreateMode)
{
	const bool bSupportsParallelResourceCreation = GRHITransientAllocatorParallelResourceCreation
#if NV_AFTERMATH
		// Aftermath adds locks that serialize placed resource creation.
		&& !UE::RHICore::Nvidia::Aftermath::IsResourceTrackingEnabled()
#endif
		;

	CreateMode = bSupportsParallelResourceCreation ? InCreateMode : ERHITransientResourceCreateMode::Inline;
}

void FRHITransientResourceHeapAllocator::Flush(FRHICommandListImmediate& RHICmdList, FRHITransientAllocationStats* OutAllocationStats)
{
	FRHITransientMemoryStats Stats;

	uint32 NumBuffers = 0;
	uint32 NumTextures = 0;

	for (FRHITransientHeap* Heap : Heaps)
	{
		Heap->Flush(CurrentCycle, Stats, OutAllocationStats);

		NumBuffers += Heap->Buffers.GetSize();
		NumTextures += Heap->Textures.GetSize();
	}

	TRACE_COUNTER_SET(TransientBufferCacheSize, NumBuffers);
	TRACE_COUNTER_SET(TransientTextureCacheSize, NumTextures);

	if (DeallocationCount > 0)
	{
		// This could be done more efficiently, but the number of heaps is small and the goal is to keep the list stable
		// so that heaps are acquired in the same order each frame, because the resource caches are tied to heaps.
		TArray<FRHITransientHeap*, FConcurrentLinearArrayAllocator> EmptyHeaps;
		TArray<FRHITransientHeap*, FConcurrentLinearArrayAllocator> ActiveHeaps;
		EmptyHeaps.Reserve(Heaps.Num());
		ActiveHeaps.Reserve(Heaps.Num());

		for (FRHITransientHeap* Heap : Heaps)
		{
			if (Heap->IsEmpty())
			{
				EmptyHeaps.Emplace(Heap);
			}
			else
			{
				ActiveHeaps.Emplace(Heap);
			}
		}

		HeapCache.Forfeit(EmptyHeaps);
		Heaps = ActiveHeaps;
		DeallocationCount = 0;
	}

	RHICmdList.EnqueueLambda([&HeapCache = HeapCache, Stats](FRHICommandListBase&)
	{
		HeapCache.Stats.Accumulate(Stats);
	});

	CurrentCycle++;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
//! Transient Resource Page Allocator
///////////////////////////////////////////////////////////////////////////////////////////////////

void FRHITransientPageSpanAllocator::Init()
{
	check(MaxSpanCount == MaxPageCount + 2);

	PageToSpanStart.AddDefaulted(MaxPageCount + 1);
	PageToSpanEnd.AddDefaulted(MaxPageCount + 1);

	PageSpans.AddDefaulted(MaxSpanCount);
	UnusedSpanList.AddDefaulted(MaxSpanCount);

	Reset();
}

void FRHITransientPageSpanAllocator::Reset()
{
	FreePageCount = MaxPageCount;
	AllocationCount = 0;

	// Initialize the unused span index pool with MaxSpanCount entries
	for (uint32 Index = 0; Index < MaxSpanCount; Index++)
	{
		UnusedSpanList[Index] = MaxSpanCount - 1 - Index;
	}
	UnusedSpanListCount = MaxSpanCount;

	// Allocate the head and tail spans (dummy spans), and a span between them covering the entire range
	uint32 HeadSpanIndex = AllocSpan();
	uint32 TailSpanIndex = AllocSpan();
	check(HeadSpanIndex == FreeSpanListHeadIndex);
	check(TailSpanIndex == FreeSpanListTailIndex);

	if (MaxPageCount > 0)
	{
		uint32 FirstFreeNodeIndex = AllocSpan();

		// Allocate head and tail nodes (0 and 1)
		for (uint32 Index = 0; Index < 2; Index++)
		{
			PageSpans[Index].Resource = nullptr;
			PageSpans[Index].Offset = 0;
			PageSpans[Index].Count = 0;
			PageSpans[Index].Fences = {};
			PageSpans[Index].NextSpanIndex = InvalidIndex;
			PageSpans[Index].PrevSpanIndex = InvalidIndex;
			PageSpans[Index].bAllocated = false;
		}
		PageSpans[HeadSpanIndex].NextSpanIndex = FirstFreeNodeIndex;
		PageSpans[TailSpanIndex].PrevSpanIndex = FirstFreeNodeIndex;

		// First Node
		PageSpans[FirstFreeNodeIndex].Resource = nullptr;
		PageSpans[FirstFreeNodeIndex].Offset = 0;
		PageSpans[FirstFreeNodeIndex].Count = MaxPageCount;
		PageSpans[FirstFreeNodeIndex].Fences = {};
		PageSpans[FirstFreeNodeIndex].PrevSpanIndex = HeadSpanIndex;
		PageSpans[FirstFreeNodeIndex].NextSpanIndex = TailSpanIndex;
		PageSpans[FirstFreeNodeIndex].bAllocated = false;

		// Initialize the page->span mapping
		for (uint32 Index = 0; Index < MaxPageCount + 1; Index++)
		{
			PageToSpanStart[Index] = InvalidIndex;
			PageToSpanEnd[Index] = InvalidIndex;
		}
		PageToSpanStart[0] = FirstFreeNodeIndex;
		PageToSpanEnd[MaxPageCount] = FirstFreeNodeIndex;
	}
	else
	{
		PageSpans[HeadSpanIndex].NextSpanIndex = TailSpanIndex;
		PageSpans[TailSpanIndex].PrevSpanIndex = HeadSpanIndex;
	}
}

bool FRHITransientPageSpanAllocator::Allocate(FRHITransientResource* Resource, const FRHITransientAllocationFences& Fences, uint32 PageCount, uint32& OutNumPagesAllocated, uint32& OutSpanIndex)
{
	OutNumPagesAllocated = 0;

	if (FreePageCount == 0)
	{
		return false;
	}

	// Allocate spans from the free list head
	uint32 NumPagesToFind = PageCount;
	uint32 FoundPages = 0;
	FPageSpan& HeadSpan = PageSpans[FreeSpanListHeadIndex];
	uint32 FirstSpanIndex = InvalidIndex;
	uint32 LastSpanIndex = InvalidIndex;
	uint32 SpanIndex = HeadSpan.NextSpanIndex;
	while (SpanIndex != FreeSpanListTailIndex && SpanIndex != InvalidIndex && NumPagesToFind > 0)
	{
		FPageSpan& Span = PageSpans[SpanIndex];
		uint32 NextSpanIndex = Span.NextSpanIndex;

		if (!FRHITransientAllocationFences::Contains(Span.Fences, Fences))
		{
			if (NumPagesToFind <= Span.Count)
			{
				// Span is too big, so split it
				if (Span.Count > NumPagesToFind)
				{
					SplitSpan(SpanIndex, NumPagesToFind);
				}
				check(NumPagesToFind == Span.Count);
			}
			Span.bAllocated = true;

			NextSpanIndex = Span.NextSpanIndex;
			Unlink(SpanIndex);
			if (FirstSpanIndex == InvalidIndex)
			{
				FirstSpanIndex = LastSpanIndex = SpanIndex;
			}
			else
			{
				InsertAfter(LastSpanIndex, SpanIndex);
				LastSpanIndex = SpanIndex;
			}

			// Record the aliasing overlap between the resource we are allocating and the one that was deallocated.
			if (Span.Resource)
			{
				Resource->AddAliasingOverlap(Span.Resource, FRHITransientAllocationFences::GetAcquireFence(Span.Fences, Fences));
			}

			check(NumPagesToFind >= Span.Count)
			NumPagesToFind -= Span.Count;
		}

		SpanIndex = NextSpanIndex;
	}

	const uint32 NumPagesAllocated = PageCount - NumPagesToFind;
	if (NumPagesAllocated > 0)
	{
		FreePageCount -= NumPagesAllocated;
		AllocationCount++;
		OutSpanIndex = FirstSpanIndex;
		OutNumPagesAllocated = NumPagesAllocated;
	}

	Validate();
	return NumPagesAllocated != 0;
}

void FRHITransientPageSpanAllocator::Deallocate(FRHITransientResource* Resource, const FRHITransientAllocationFences& Fences, uint32 SpanIndex)
{
	if (SpanIndex == InvalidIndex)
	{
		return;
	}
	check(AllocationCount > 0);
	while (SpanIndex != InvalidIndex)
	{
		FPageSpan& FreedSpan = PageSpans[SpanIndex];
		check(FreedSpan.bAllocated);
		FreePageCount += FreedSpan.Count;
		uint32 NextSpanIndex = FreedSpan.NextSpanIndex;
		FreedSpan.Resource = Resource;
		FreedSpan.Fences = Fences;
		FreedSpan.bAllocated = false;
		Unlink(SpanIndex);
		InsertAfter(FreeSpanListHeadIndex, SpanIndex);
		SpanIndex = NextSpanIndex;
	}
	AllocationCount--;

	Validate();
}

void FRHITransientPageSpanAllocator::SplitSpan(uint32 InSpanIndex, uint32 InPageCount)
{
	FPageSpan& Span = PageSpans[InSpanIndex];
	check(InPageCount <= Span.Count);
	if (InPageCount < Span.Count)
	{
		uint32 NewSpanIndex = AllocSpan();
		FPageSpan& NewSpan = PageSpans[NewSpanIndex];
		NewSpan.Resource = Span.Resource;
		NewSpan.Fences = Span.Fences;
		NewSpan.NextSpanIndex = Span.NextSpanIndex;
		NewSpan.PrevSpanIndex = InSpanIndex;
		NewSpan.Count = Span.Count - InPageCount;
		NewSpan.Offset = Span.Offset + InPageCount;
		NewSpan.bAllocated = Span.bAllocated;
		Span.Count = InPageCount;
		Span.NextSpanIndex = NewSpanIndex;
		if (NewSpan.NextSpanIndex != InvalidIndex)
		{
			PageSpans[NewSpan.NextSpanIndex].PrevSpanIndex = NewSpanIndex;
		}

		// Update the PageToSpan mappings
		PageToSpanEnd[NewSpan.Offset] = InSpanIndex;
		PageToSpanStart[NewSpan.Offset] = NewSpanIndex;
		PageToSpanEnd[NewSpan.Offset + NewSpan.Count] = NewSpanIndex;
	}
}

void FRHITransientPageSpanAllocator::MergeSpans(uint32 SpanIndex0, uint32 SpanIndex1)
{
	FPageSpan& Span0 = PageSpans[SpanIndex0];
	FPageSpan& Span1 = PageSpans[SpanIndex1];
	check(Span0.Offset + Span0.Count == Span1.Offset);
	check(Span0.bAllocated == Span1.bAllocated);
	check(Span0.NextSpanIndex == SpanIndex1);
	check(Span1.PrevSpanIndex == SpanIndex0);

	uint32 SpanIndexToKeep = SpanIndex0;
	uint32 SpanIndexToRemove = SpanIndex1;

	// Update the PageToSpan mappings
	PageToSpanStart[Span0.Offset] = SpanIndexToKeep;
	PageToSpanStart[Span1.Offset] = InvalidIndex;
	PageToSpanEnd[Span0.Offset + Span0.Count] = InvalidIndex; // Should match Span1.Offset
	PageToSpanEnd[Span1.Offset + Span1.Count] = SpanIndexToKeep;
	Span0.Count += Span1.Count;

	Unlink(SpanIndexToRemove);
	ReleaseSpan(SpanIndexToRemove);
}

void FRHITransientPageSpanAllocator::Flush()
{
	uint32 PageIndex = 0;
	while (PageIndex < MaxPageCount)
	{
		int32 SpanIndex = PageToSpanStart[PageIndex];
		check(SpanIndex != InvalidIndex);
		FPageSpan& Span = PageSpans[SpanIndex];

		if (!Span.bAllocated)
		{
			Span.Resource = nullptr;
			Span.Fences = {};

			while (true)
			{
				// Can we merge this span with an existing free one to the right?
				int32 NextSpanIndex = PageToSpanStart[Span.Offset + Span.Count];

				if (NextSpanIndex == InvalidIndex)
				{
					break;
				}

				FPageSpan& NextPageSpan = PageSpans[NextSpanIndex];

				if (NextPageSpan.bAllocated)
				{
					PageIndex += NextPageSpan.Count;
					break;
				}

				NextPageSpan.Resource = nullptr;
				Unlink(SpanIndex);
				InsertBefore(NextSpanIndex, SpanIndex);
				MergeSpans(SpanIndex, NextSpanIndex);
				Validate();
			}
		}

		PageIndex += Span.Count;
	}
}

// Inserts a span after an existing span. The span to insert must be unlinked
void FRHITransientPageSpanAllocator::InsertAfter(uint32 InsertPosition, uint32 InsertSpanIndex)
{
	check(InsertPosition != InvalidIndex);
	check(InsertSpanIndex != InvalidIndex);
	FPageSpan& SpanAtPos = PageSpans[InsertPosition];
	FPageSpan& SpanToInsert = PageSpans[InsertSpanIndex];
	check(!SpanToInsert.IsLinked());

	// Connect Span0's next node with the inserted node
	SpanToInsert.NextSpanIndex = SpanAtPos.NextSpanIndex;
	if (SpanAtPos.NextSpanIndex != InvalidIndex)
	{
		PageSpans[SpanAtPos.NextSpanIndex].PrevSpanIndex = InsertSpanIndex;
	}
	// Connect the two nodes
	SpanAtPos.NextSpanIndex = InsertSpanIndex;
	SpanToInsert.PrevSpanIndex = InsertPosition;
}

// Inserts a span after an existing span. The span to insert must be unlinked
void FRHITransientPageSpanAllocator::InsertBefore(uint32 InsertPosition, uint32 InsertSpanIndex)
{
	check(InsertPosition != InvalidIndex && InsertPosition != 0); // Can't insert before the head
	check(InsertSpanIndex != InvalidIndex);
	FPageSpan& SpanAtPos = PageSpans[InsertPosition];
	FPageSpan& SpanToInsert = PageSpans[InsertSpanIndex];
	check(!SpanToInsert.IsLinked());

	// Connect Span0's prev node with the inserted node
	SpanToInsert.PrevSpanIndex = SpanAtPos.PrevSpanIndex;
	if (SpanAtPos.PrevSpanIndex != InvalidIndex)
	{
		PageSpans[SpanAtPos.PrevSpanIndex].NextSpanIndex = InsertSpanIndex;
	}

	// Connect the two nodes
	SpanAtPos.PrevSpanIndex = InsertSpanIndex;
	SpanToInsert.NextSpanIndex = InsertPosition;
}

void FRHITransientPageSpanAllocator::Unlink(uint32 SpanIndex)
{
	FPageSpan& Span = PageSpans[SpanIndex];
	check(SpanIndex != FreeSpanListHeadIndex);
	if (Span.PrevSpanIndex != InvalidIndex)
	{
		PageSpans[Span.PrevSpanIndex].NextSpanIndex = Span.NextSpanIndex;
	}
	if (Span.NextSpanIndex != InvalidIndex)
	{
		PageSpans[Span.NextSpanIndex].PrevSpanIndex = Span.PrevSpanIndex;
	}
	Span.PrevSpanIndex = InvalidIndex;
	Span.NextSpanIndex = InvalidIndex;
}

uint32 FRHITransientPageSpanAllocator::GetAllocationPageCount(uint32 SpanIndex) const
{
	check(SpanIndex != InvalidIndex && SpanIndex < MaxSpanCount);
	check(PageSpans[SpanIndex].bAllocated);

	uint32 Count = 0;
	do
	{
		Count += PageSpans[SpanIndex].Count;
		SpanIndex = PageSpans[SpanIndex].NextSpanIndex;

	} while (SpanIndex != InvalidIndex);

	return Count;
}

void FRHITransientPageSpanAllocator::Validate()
{
#if UE_BUILD_DEBUG
	// Check the mappings are valid
	for (uint32 Index = 0; Index < MaxPageCount; Index++)
	{
		check(PageToSpanStart[Index] == InvalidIndex || PageSpans[PageToSpanStart[Index]].Offset == Index);
		check(PageToSpanEnd[Index] == InvalidIndex || PageSpans[PageToSpanEnd[Index]].Offset + PageSpans[PageToSpanEnd[Index]].Count == Index);
	}

	// Count free pages
	uint32 FreeCount = 0;
	uint32 PrevIndex = FreeSpanListHeadIndex;
	for (uint32 Index = GetFirstSpanIndex(); PageSpans.IsValidIndex(Index); Index = PageSpans[Index].NextSpanIndex)
	{
		FPageSpan& Span = PageSpans[Index];
		check(Span.PrevSpanIndex == PrevIndex);
		check(Index == FreeSpanListHeadIndex || Index == FreeSpanListTailIndex || Span.Count != 0);
		PrevIndex = Index;
		FreeCount += Span.Count;
	}
	check(FreeCount <= MaxPageCount);
	check(FreeCount == FreePageCount);
#endif
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FRHITransientPagePool::Allocate(FAllocationContext& Context)
{
	uint32 SpanIndex = 0;
	uint32 PagesAllocated = 0;

	uint32 PagesRemaining = Context.MaxAllocationPage > 0 ? FMath::Min(Context.PagesRemaining, Context.MaxAllocationPage) : Context.PagesRemaining;
	if (Allocator.Allocate(&Context.Resource, Context.Fences, PagesRemaining, PagesAllocated, SpanIndex))
	{
		const uint64 DestinationGpuVirtualAddress = Context.GpuVirtualAddress + Context.PagesAllocated * Initializer.PageSize;
		const uint32 PageSpanOffsetMin = PageSpans.Num();

		Allocator.GetSpanArray(SpanIndex, PageSpans);

		const uint32 PageSpanOffsetMax = PageSpans.Num();
		const uint32 PageSpanCount     = PageSpanOffsetMax - PageSpanOffsetMin;

		const  int32 AllocationIndex   = Context.Allocations.Num();
		const uint64 AllocationHash    = CityHash64WithSeed((const char*)&PageSpans[PageSpanOffsetMin], PageSpanCount * sizeof(FRHITransientPageSpan), DestinationGpuVirtualAddress);

		FRHITransientPagePoolAllocation Allocation;
		Allocation.Pool = this;
		Allocation.Hash = AllocationHash;
		Allocation.SpanOffsetMin = Context.Spans.Num();
		Allocation.SpanOffsetMax = Context.Spans.Num() + PageSpanCount;
		Allocation.SpanIndex = SpanIndex;
		Context.Allocations.Emplace(Allocation);
		Context.Spans.Append(&PageSpans[PageSpanOffsetMin], PageSpanCount);

		bool bMapPages = true;

		if (AllocationIndex < Context.AllocationsBefore.Num())
		{
			const FRHITransientPagePoolAllocation& AllocationBefore = Context.AllocationsBefore[AllocationIndex];

			if (AllocationBefore.Hash == AllocationHash && AllocationBefore.Pool == this)
			{
				Context.AllocationMatchingCount++;
				bMapPages = false;
			}
		}

		if (bMapPages)
		{
			PageMapRequests.Emplace(DestinationGpuVirtualAddress, GpuVirtualAddress, Initializer.PageCount, PageSpanOffsetMin, PageSpanCount);
			Context.PagesMapped += PagesAllocated;
		}

		check(Context.PagesRemaining >= PagesAllocated);
		Context.PagesRemaining -= PagesAllocated;
		Context.PagesAllocated += PagesAllocated;
		Context.PageSpansAllocated += PageSpanCount;
		Context.AllocationCount++;
	}
}

void FRHITransientPagePool::Flush(FRHICommandListImmediate& RHICmdList)
{
	if (!PageMapRequests.IsEmpty())
	{
		PageMapRequestCountMax = FMath::Max<uint32>(PageMapRequests.Num(), PageMapRequestCountMax);
		PageSpanCountMax       = FMath::Max<uint32>(PageSpans.Num(),       PageSpanCountMax);

		Flush(RHICmdList, MoveTemp(PageMapRequests), MoveTemp(PageSpans));

		PageMapRequests.Reserve(PageMapRequestCountMax);
		PageSpans.Reserve(PageSpanCountMax);
	}

	Allocator.Flush();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FRHITransientPagePoolCache::FInitializer FRHITransientPagePoolCache::FInitializer::CreateDefault()
{
	FInitializer Initializer;
	Initializer.BufferCacheSize = GRHITransientAllocatorBufferCacheSize;
	Initializer.TextureCacheSize = GRHITransientAllocatorTextureCacheSize;
	Initializer.GarbageCollectLatency = GRHITransientAllocatorGarbageCollectLatency;
	return Initializer;
}

FRHITransientPagePoolCache::~FRHITransientPagePoolCache()
{
	delete FastPagePool;
	FastPagePool = nullptr;

	for (FRHITransientPagePool* PagePool : LiveList)
	{
		delete PagePool;
	}
	LiveList.Empty();
	FreeList.Empty();
}

FRHITransientPagePool* FRHITransientPagePoolCache::Acquire()
{
	FScopeLock Lock(&CriticalSection);

	if (!FreeList.IsEmpty())
	{
		return FreeList.Pop();
	}

	LLM_SCOPE_BYTAG(RHITransientResources);

	FRHITransientPagePool::FInitializer PagePoolInitializer;
	PagePoolInitializer.PageSize = Initializer.PageSize;

	if (LiveList.IsEmpty() && Initializer.PoolSizeFirst > Initializer.PoolSize)
	{
		PagePoolInitializer.PageCount = Initializer.PoolSizeFirst / Initializer.PageSize;
	}
	else
	{
		PagePoolInitializer.PageCount = Initializer.PoolSize / Initializer.PageSize;
	}

	FRHITransientPagePool* PagePool = CreatePagePool(PagePoolInitializer);
	check(PagePool);

	TotalMemoryCapacity += PagePool->GetCapacity();
	LiveList.Emplace(PagePool);

	return PagePool;
}

FRHITransientPagePool* FRHITransientPagePoolCache::GetFastPagePool()
{
	if (!FastPagePool)
	{
		LLM_SCOPE_BYTAG(RHITransientResources);
		FastPagePool = CreateFastPagePool();

		if (FastPagePool)
		{
			TotalMemoryCapacity += FastPagePool->GetCapacity();
			return FastPagePool;
		}
	}

	return nullptr;
}

void FRHITransientPagePoolCache::Forfeit(TConstArrayView<FRHITransientPagePool*> InForfeitedPagePools)
{
	FScopeLock Lock(&CriticalSection);

	// These are iterated in reverse so they are acquired in the same order. 
	for (int32 Index = InForfeitedPagePools.Num() - 1; Index >= 0; --Index)
	{
		FRHITransientPagePool* PagePool = InForfeitedPagePools[Index];
		check(PagePool->IsEmpty());
		PagePool->LastUsedGarbageCollectCycle = GarbageCollectCycle;
		FreeList.Add(PagePool);
	}
}

void FRHITransientPagePoolCache::GarbageCollect()
{
	SCOPED_NAMED_EVENT_TEXT("TransientPagePoolCache::GarbageCollect", FColor::Magenta);
	TArray<FRHITransientPagePool*, TInlineAllocator<16>> PoolsToDelete;

	{
		FScopeLock Lock(&CriticalSection);

		for (int32 PagePoolIndex = 0; PagePoolIndex < FreeList.Num(); ++PagePoolIndex)
		{
			FRHITransientPagePool* PagePool = FreeList[PagePoolIndex];

			if (PagePool->GetLastUsedGarbageCollectCycle() + Initializer.GarbageCollectLatency <= GarbageCollectCycle)
			{
				TotalMemoryCapacity -= PagePool->GetCapacity();
				FreeList.RemoveAt(PagePoolIndex);
				LiveList.Remove(PagePool);
				PagePoolIndex--;

				PoolsToDelete.Emplace(PagePool);

				// Only delete one per frame. Deletion can be quite expensive.
				break;
			}
		}

		TRACE_COUNTER_SET(TransientMemoryRangeCount, LiveList.Num());
	}

	Stats.Submit(TotalMemoryCapacity);

	GarbageCollectCycle++;

	for (FRHITransientPagePool* PagePool : PoolsToDelete)
	{
		delete PagePool;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FRHITransientTexture* FRHITransientResourcePageAllocator::CreateTexture(
	const FRHITextureCreateInfo& CreateInfo,
	const TCHAR* DebugName,
	const FRHITransientAllocationFences& Fences)
{
	FRHITransientTexture* Texture = Textures.Acquire(ComputeHash(CreateInfo), [&](uint64 Hash)
	{
		Stats.Textures.CreateCount++;
		return CreateTextureInternal(CreateInfo, DebugName, Hash);
	});

	const bool bFastPool = EnumHasAnyFlags(CreateInfo.Flags, ETextureCreateFlags::FastVRAM) || EnumHasAnyFlags(CreateInfo.Flags, ETextureCreateFlags::FastVRAMPartialAlloc);
	const float FastPoolPercentageRequested = bFastPool ? CreateInfo.FastVRAMPercentage / 255.f : 0.f;

	check(Texture);
	Texture->Acquire(DebugName, Fences.GetSinglePipeline(), Fences.GetPipelines(), CurrentCycle);
	AllocateMemoryInternal(Texture, DebugName, Fences, bFastPool, FastPoolPercentageRequested);
	Stats.AllocateTexture(Texture->GetSize());
	IF_RHICORE_TRANSIENT_ALLOCATOR_DEBUG(ActiveResources.Emplace(Texture));
	return Texture;
}

FRHITransientBuffer* FRHITransientResourcePageAllocator::CreateBuffer(
	const FRHIBufferCreateInfo& CreateInfo,
	const TCHAR* DebugName,
	const FRHITransientAllocationFences& Fences)
{
	FRHITransientBuffer* Buffer = Buffers.Acquire(ComputeHash(CreateInfo), [&](uint64 Hash)
	{
		Stats.Buffers.CreateCount++;
		return CreateBufferInternal(CreateInfo, DebugName, Hash);
	});

	check(Buffer);
	Buffer->Acquire(DebugName, Fences.GetSinglePipeline(), Fences.GetPipelines(), CurrentCycle);
	AllocateMemoryInternal(Buffer, DebugName, Fences, EnumHasAnyFlags(CreateInfo.Usage, EBufferUsageFlags::FastVRAM), false);
	Stats.AllocateBuffer(Buffer->GetSize());
	IF_RHICORE_TRANSIENT_ALLOCATOR_DEBUG(ActiveResources.Emplace(Buffer));
	return Buffer;
}

void FRHITransientResourcePageAllocator::AllocateMemoryInternal(FRHITransientResource* Resource, const TCHAR* DebugName, const FRHITransientAllocationFences& Fences, bool bFastPoolRequested, float FastPoolPercentageRequested)
{
	FRHITransientPagePool::FAllocationContext AllocationContext(*Resource, Fences, PageSize);

	if (bFastPoolRequested && FastPagePool)
	{
		// If a partial allocation is requested, compute the max. number of page which should be allocated in fast memory
		AllocationContext.MaxAllocationPage = FastPoolPercentageRequested > 0 ? FMath::CeilToInt(AllocationContext.PagesRemaining * FastPoolPercentageRequested) : AllocationContext.MaxAllocationPage;
		FastPagePool->Allocate(AllocationContext);
		AllocationContext.MaxAllocationPage = 0;
	}

	if (!AllocationContext.IsComplete())
	{
		for (FRHITransientPagePool* PagePool : PagePools)
		{
			PagePool->Allocate(AllocationContext);

			if (AllocationContext.IsComplete())
			{
				break;
			}
		}
	}

	while (!AllocationContext.IsComplete())
	{
		FRHITransientPagePool* PagePool = PagePoolCache.Acquire();
		PagePool->Allocate(AllocationContext);
		PagePools.Emplace(PagePool);
	}

	PageMapCount      += AllocationContext.PagesMapped;
	PageAllocateCount += AllocationContext.PagesAllocated;
	PageSpanCount     += AllocationContext.PageSpansAllocated;
}

void FRHITransientResourcePageAllocator::DeallocateMemoryInternal(FRHITransientResource* Resource, const FRHITransientAllocationFences& Fences)
{
	Resource->Discard(Fences);

	for (const FRHITransientPagePoolAllocation& Allocation : Resource->GetPageAllocation().PoolAllocations)
	{
		Allocation.Pool->Deallocate(Resource, Fences, Allocation.SpanIndex);
	}
}

void FRHITransientResourcePageAllocator::DeallocateMemory(FRHITransientTexture* Texture, const FRHITransientAllocationFences& Fences)
{
	DeallocateMemoryInternal(Texture, Fences);
	Stats.DeallocateTexture(Texture->GetSize());
}

void FRHITransientResourcePageAllocator::DeallocateMemory(FRHITransientBuffer* Buffer, const FRHITransientAllocationFences& Fences)
{
	DeallocateMemoryInternal(Buffer, Fences);
	Stats.DeallocateBuffer(Buffer->GetSize());
}

void FRHITransientResourcePageAllocator::Flush(FRHICommandListImmediate& RHICmdList, FRHITransientAllocationStats* OutAllocationStats)
{
	if (OutAllocationStats)
	{
		TMap<FRHITransientPagePool*, int32> PagePoolToMemoryRangeIndex;
		PagePoolToMemoryRangeIndex.Reserve(PagePools.Num() + !!FastPagePool);

		const auto AddMemoryRange = [&](FRHITransientPagePool* PagePool, FRHITransientAllocationStats::EMemoryRangeFlags Flags)
		{
			PagePoolToMemoryRangeIndex.Emplace(PagePool, OutAllocationStats->MemoryRanges.Num());

			FRHITransientAllocationStats::FMemoryRange MemoryRange;
			MemoryRange.Capacity = PagePool->GetCapacity();
			MemoryRange.CommitSize = PagePool->GetCapacity();
			MemoryRange.Flags = Flags;
			OutAllocationStats->MemoryRanges.Emplace(MemoryRange);
		};

		if (FastPagePool)
		{
			AddMemoryRange(FastPagePool, FRHITransientAllocationStats::EMemoryRangeFlags::FastVRAM);
		}

		for (FRHITransientPagePool* PagePool : PagePools)
		{
			AddMemoryRange(PagePool, FRHITransientAllocationStats::EMemoryRangeFlags::None);
		}

		for (const FRHITransientTexture* Texture : Textures.GetAllocated())
		{
			FRHITransientAllocationStats::FAllocationArray& Allocations = OutAllocationStats->Resources.Emplace(Texture);

			EnumeratePageSpans(Texture, [&](FRHITransientPagePool* PagePool, FRHITransientPageSpan PageSpan)
			{
				FRHITransientAllocationStats::FAllocation Allocation;
				Allocation.OffsetMin = PageSize *  PageSpan.Offset;
				Allocation.OffsetMax = PageSize * (PageSpan.Offset + PageSpan.Count);
				Allocation.MemoryRangeIndex = PagePoolToMemoryRangeIndex[PagePool];
				Allocations.Emplace(Allocation);
			});
		}

		for (const FRHITransientBuffer* Buffer : Buffers.GetAllocated())
		{
			FRHITransientAllocationStats::FAllocationArray& Allocations = OutAllocationStats->Resources.Emplace(Buffer);

			EnumeratePageSpans(Buffer, [&](FRHITransientPagePool* PagePool, FRHITransientPageSpan PageSpan)
			{
				FRHITransientAllocationStats::FAllocation Allocation;
				Allocation.OffsetMin = PageSize *  PageSpan.Offset;
				Allocation.OffsetMax = PageSize * (PageSpan.Offset + PageSpan.Count);
				Allocation.MemoryRangeIndex = PagePoolToMemoryRangeIndex[PagePool];
				Allocations.Emplace(Allocation);
			});
		}
	}

	{
		TRACE_COUNTER_SET(TransientPageMapCount, PageMapCount);
		TRACE_COUNTER_SET(TransientPageAllocateCount, PageAllocateCount);
		TRACE_COUNTER_SET(TransientPageSpanCount, PageSpanCount);
		PageMapCount = 0;
		PageAllocateCount = 0;
		PageSpanCount = 0;
	}

	{
		TRACE_COUNTER_SET(TransientTextureCount, Textures.GetAllocatedCount());
		TRACE_COUNTER_SET(TransientTextureCacheHitPercentage, Textures.GetHitPercentage());

		Textures.Forfeit(GFrameCounterRenderThread, [this](FRHITransientTexture* Texture) { ReleaseTextureInternal(Texture); });

		TRACE_COUNTER_SET(TransientTextureCacheSize, Textures.GetSize());
	}

	{
		TRACE_COUNTER_SET(TransientBufferCount, Buffers.GetAllocatedCount());
		TRACE_COUNTER_SET(TransientBufferCacheHitPercentage, Buffers.GetHitPercentage());

		Buffers.Forfeit(GFrameCounterRenderThread, [this](FRHITransientBuffer* Buffer) { ReleaseBufferInternal(Buffer); });

		TRACE_COUNTER_SET(TransientBufferCacheSize, Buffers.GetSize());
	}

	if (FastPagePool)
	{
		FastPagePool->Flush(RHICmdList);
	}

	for (FRHITransientPagePool* PagePool : PagePools)
	{
		PagePool->Flush(RHICmdList);
	}

	if (Stats.HasDeallocations())
	{
		TArray<FRHITransientPagePool*, FConcurrentLinearArrayAllocator> EmptyPagePools;
		TArray<FRHITransientPagePool*, FConcurrentLinearArrayAllocator> ActivePagePools;
		EmptyPagePools.Reserve(PagePools.Num());
		ActivePagePools.Reserve(PagePools.Num());

		for (FRHITransientPagePool* PagePool : PagePools)
		{
			if (PagePool->IsEmpty())
			{
				EmptyPagePools.Emplace(PagePool);
			}
			else
			{
				ActivePagePools.Emplace(PagePool);
			}
		}

		PagePoolCache.Forfeit(EmptyPagePools);
		PagePools = ActivePagePools;
	}

	RHICmdList.EnqueueLambda([&PagePoolCache = PagePoolCache, Stats = Stats](FRHICommandListBase&)
	{
		PagePoolCache.Stats.Accumulate(Stats);
	});

	Stats.Reset();

	CurrentCycle++;
}