// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIDescriptorAllocator.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformStackWalk.h"
#include "Misc/ScopeLock.h"
#include "RHICore.h"
#include "RHIDefinitions.h"

#if DO_CHECK
static bool GRHITrackOutstandingDescriptorAllocations = false;
static FAutoConsoleVariableRef CVarRHITrackOutstandingDescriptorAllocations(
	TEXT("rhi.TrackOutstandingDescriptorAllocations"),
	GRHITrackOutstandingDescriptorAllocations,
	TEXT("When enabled, tracks outstanding descriptor allocations to detect double-free and leak bugs."),
	ECVF_ReadOnly
);

static bool GRHITrackDescriptorCallstacks = false;
static FAutoConsoleVariableRef CVarRHITrackDescriptorCallstacks(
	TEXT("rhi.TrackDescriptorCallstacks"),
	GRHITrackDescriptorCallstacks,
	TEXT("When enabled, captures callstacks for descriptor allocations to help diagnose double-free bugs. Requires rhi.TrackOutstandingDescriptorAllocations."),
	ECVF_ReadOnly
);
#endif

FRHIDescriptorAllocator::FRHIDescriptorAllocator()
{
}

FRHIDescriptorAllocator::FRHIDescriptorAllocator(uint32 InNumDescriptors, TConstArrayView<TStatId> InStats)
{
	Init(InNumDescriptors, InStats);
}

FRHIDescriptorAllocator::~FRHIDescriptorAllocator()
{
#if DO_CHECK
	if (GRHITrackOutstandingDescriptorAllocations && !DebugOutstandingAllocations.IsEmpty())
	{
		UE_LOGF(LogRHICore, Error, "FRHIDescriptorAllocator destroyed with %d outstanding allocations. Possible descriptor leak.",
			DebugOutstandingAllocations.Num());

		for (const TPair<uint32, FDebugDescriptorAllocation>& Pair : DebugOutstandingAllocations)
		{
			UE_LOGF(LogRHICore, Error, "  Leaked allocation: Offset=%u, Count=%u", Pair.Key, Pair.Value.Count);
			for (uint64 Address : Pair.Value.Callstack)
			{
				FProgramCounterSymbolInfo SymbolInfo;
				FPlatformStackWalk::ProgramCounterToSymbolInfo(Address, SymbolInfo);
				ANSICHAR HumanReadableString[1024] = {};
				FPlatformStackWalk::SymbolInfoToHumanReadableString(SymbolInfo, HumanReadableString, UE_ARRAY_COUNT(HumanReadableString));
				UE_LOGF(LogRHICore, Error, "    %s", HumanReadableString);
			}
		}

		checkf(false, TEXT("FRHIDescriptorAllocator destroyed with %d outstanding allocations. See log above for details."),
			DebugOutstandingAllocations.Num());
	}
#endif
}

void FRHIDescriptorAllocator::Init(uint32 InNumDescriptors, TConstArrayView<TStatId> InStats)
{
	Capacity = InNumDescriptors;
	Ranges.Emplace(0, InNumDescriptors - 1);

#if STATS
	Stats = InStats;
#endif
}

void FRHIDescriptorAllocator::Shutdown()
{
	Ranges.Empty();
	Capacity = 0;
}

TOptional<FRHIDescriptorAllocation> FRHIDescriptorAllocator::ResizeGrowAndAllocate(uint32 NewCapacity, uint32 NumAllocations)
{
	check(Capacity < NewCapacity);

	FScopeLock Lock(&CriticalSection);

	bool bAddEndRange = true;
	if (Ranges.Num() > 0)
	{
		// Check if it can be merged with the last range
		FRHIDescriptorAllocatorRange& LastRange = Ranges[Ranges.Num() - 1];
		const uint32 VeryLastIndex = GetCapacity() - 1;
		if (LastRange.Last == VeryLastIndex)
		{
			LastRange.Last = NewCapacity - 1;
			bAddEndRange = false;
		}
	}

	// Add a new range at the end
	if (bAddEndRange)
	{
		Ranges.Emplace(Capacity, NewCapacity - 1);
	}

	Capacity = NewCapacity;

	TOptional<FRHIDescriptorAllocation> Allocation = Allocate(NumAllocations);
	verify(Allocation);
	return Allocation;
}

TOptional<FRHIDescriptorAllocation> FRHIDescriptorAllocator::Allocate(uint32 NumDescriptors)
{
	FScopeLock Lock(&CriticalSection);
	return AllocateInternal(NumDescriptors);
}

FRHIDescriptorHandle FRHIDescriptorAllocator::Allocate(ERHIDescriptorType InType)
{
	if (TOptional<FRHIDescriptorAllocation> Allocation = Allocate(1))
	{
		return FRHIDescriptorHandle(InType, Allocation->StartIndex);
	}
	return FRHIDescriptorHandle();
}

TOptional<FRHIDescriptorAllocation> FRHIDescriptorAllocator::AllocateInternal(uint32 NumDescriptors)
{
	if (const uint32 NumRanges = Ranges.Num(); NumRanges > 0)
	{
		uint32 Index = 0;
		do
		{
			FRHIDescriptorAllocatorRange& CurrentRange = Ranges[Index];
			const uint32 Size = 1 + CurrentRange.Last - CurrentRange.First;
			if (NumDescriptors <= Size)
			{
				uint32 First = CurrentRange.First;
				if (NumDescriptors == Size && Index + 1 < NumRanges)
				{
					// Range is full and a new range exists, so move on to that one
					Ranges.RemoveAt(Index);
				}
				else
				{
					CurrentRange.First += NumDescriptors;
				}

				RecordAlloc(NumDescriptors);

#if DO_CHECK
				if (GRHITrackOutstandingDescriptorAllocations)
				{
					FDebugDescriptorAllocation& DebugAlloc = DebugOutstandingAllocations.Add(First);
					DebugAlloc.Count = NumDescriptors;
					if (GRHITrackDescriptorCallstacks)
					{
						DebugAlloc.Callstack.SetNumUninitialized(16);
						uint32 Depth = FPlatformStackWalk::CaptureStackBackTrace(reinterpret_cast<uint64*>(DebugAlloc.Callstack.GetData()), 16);
						DebugAlloc.Callstack.SetNum(Depth);
					}
				}
#endif

				return FRHIDescriptorAllocation(First, NumDescriptors);
			}
			++Index;
		} while (Index < NumRanges);
	}

	return TOptional<FRHIDescriptorAllocation>();
}

void FRHIDescriptorAllocator::FreeInternal(FRHIDescriptorAllocation Allocation)
{
	const uint32 Offset = Allocation.StartIndex;
	const uint32 NumDescriptors = Allocation.Count;

	if (Offset == UINT_MAX || NumDescriptors == 0)
	{
		return;
	}

#if DO_CHECK
	if (GRHITrackOutstandingDescriptorAllocations)
	{
		FDebugDescriptorAllocation RemovedAlloc;
		if (!DebugOutstandingAllocations.RemoveAndCopyValue(Offset, RemovedAlloc))
		{
			UE_LOGF(LogRHICore, Fatal, "FRHIDescriptorAllocator::Free - Allocation at offset %u (count %u) was not found in outstanding allocations. Possible double-free.",
				Offset, NumDescriptors);
		}
		else if (RemovedAlloc.Count != NumDescriptors)
		{
			UE_LOGF(LogRHICore, Error, "FRHIDescriptorAllocator::Free - Allocation at offset %u has mismatched count: freeing %u but allocated %u. Original allocation callstack:",
				Offset, NumDescriptors, RemovedAlloc.Count);
			for (uint64 Address : RemovedAlloc.Callstack)
			{
				FProgramCounterSymbolInfo SymbolInfo;
				FPlatformStackWalk::ProgramCounterToSymbolInfo(Address, SymbolInfo);
				ANSICHAR HumanReadableString[1024] = {};
				FPlatformStackWalk::SymbolInfoToHumanReadableString(SymbolInfo, HumanReadableString, UE_ARRAY_COUNT(HumanReadableString));
				UE_LOGF(LogRHICore, Error, "    %s", HumanReadableString);
			}
			UE_LOGF(LogRHICore, Fatal, "FRHIDescriptorAllocator::Free - Mismatched allocation count.");
		}
	}
#endif

	const uint32 End = Offset + NumDescriptors;
	// Binary search of the range list
	uint32 Index0 = 0;
	uint32 Index1 = Ranges.Num() - 1;
	for (;;)
	{
		const uint32 Index = (Index0 + Index1) / 2;
		if (Offset < Ranges[Index].First)
		{
			// Before current range, check if neighboring
			if (End >= Ranges[Index].First)
			{
				check(End == Ranges[Index].First); // Can't overlap a range of free IDs
				// Neighbor id, check if neighboring previous range too
				if (Index > Index0 && Offset - 1 == Ranges[Index - 1].Last)
				{
					// Merge with previous range
					Ranges[Index - 1].Last = Ranges[Index].Last;
					Ranges.RemoveAt(Index);
				}
				else
				{
					// Just grow range
					Ranges[Index].First = Offset;
				}

				RecordFree(NumDescriptors);
				return;
			}
			else
			{
				// Non-neighbor id
				if (Index != Index0)
				{
					// Cull upper half of list
					Index1 = Index - 1;
				}
				else
				{
					// Found our position in the list, insert the deleted range here
					Ranges.EmplaceAt(Index, Offset, End - 1);

					RecordFree(NumDescriptors);
					return;
				}
			}
		}
		else if (Offset > Ranges[Index].Last)
		{
			// After current range, check if neighboring
			if (Offset - 1 == Ranges[Index].Last)
			{
				// Neighbor id, check if neighboring next range too
				if (Index < Index1 && End == Ranges[Index + 1].First)
				{
					// Merge with next range
					Ranges[Index].Last = Ranges[Index + 1].Last;
					Ranges.RemoveAt(Index + 1);
				}
				else
				{
					// Just grow range
					Ranges[Index].Last += NumDescriptors;
				}

				RecordFree(NumDescriptors);
				return;
			}
			else
			{
				// Non-neighbor id
				if (Index != Index1)
				{
					// Cull bottom half of list
					Index0 = Index + 1;
				}
				else
				{
					// Found our position in the list, insert the deleted range here
					Ranges.EmplaceAt(Index + 1, Offset, End - 1);

					RecordFree(NumDescriptors);
					return;
				}
			}
		}
		else
		{
			// Inside a free block, not a valid offset
			checkNoEntry();
		}
	}
}

void FRHIDescriptorAllocator::Free(FRHIDescriptorAllocation Allocation)
{
	FScopeLock Lock(&CriticalSection);
	FreeInternal(Allocation);
}

void FRHIDescriptorAllocator::Free(TConstArrayView<FRHIDescriptorHandle> InHandles)
{
	if (InHandles.Num() == 0)
	{
		return;
	}

	FScopeLock Lock(&CriticalSection);
	for (FRHIDescriptorHandle Handle : InHandles)
	{
		if (Handle.IsValid())
		{
			FreeInternal(FRHIDescriptorAllocation(Handle.GetIndex(), 1));
		}
	}
}

void FRHIDescriptorAllocator::Free(FRHIDescriptorHandle InHandle)
{
	if (InHandle.IsValid())
	{
		FScopeLock Lock(&CriticalSection);
		FreeInternal(FRHIDescriptorAllocation(InHandle.GetIndex(), 1));
	}
}

bool FRHIDescriptorAllocator::GetAllocatedRange(FRHIDescriptorAllocatorRange& OutRange)
{
	const uint32 VeryFirstIndex = 0;
	const uint32 VeryLastIndex = GetCapacity() - 1;

	OutRange.First = VeryFirstIndex;
	OutRange.Last = VeryLastIndex;

	FScopeLock Lock(&CriticalSection);
	if (Ranges.Num() > 0)
	{
		const FRHIDescriptorAllocatorRange FirstRange = Ranges[0];

		// If the free range matches the entire usable range, that means we have zero allocations.
		if (FirstRange.First == VeryFirstIndex && FirstRange.Last == VeryLastIndex)
		{
			return false;
		}

		// If the first free range is at the start, then the first allocation is right after this range
		if (FirstRange.First == VeryFirstIndex)
		{
			OutRange.First = FMath::Min(FirstRange.Last + 1, VeryLastIndex);
		}

		const FRHIDescriptorAllocatorRange LastRange = Ranges[Ranges.Num() - 1];

		// If the last free range is at the end of the usable range, our last allocation is right before this range 
		if (LastRange.Last == VeryLastIndex)
		{
			OutRange.Last = LastRange.First > 0 ? LastRange.First - 1 : 0;
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FRHIHeapDescriptorAllocator

FRHIHeapDescriptorAllocator::FRHIHeapDescriptorAllocator(ERHIDescriptorTypeMask InTypeMask, uint32 InDescriptorCount, TConstArrayView<TStatId> InStats)
	: FRHIDescriptorAllocator(InDescriptorCount, InStats)
	, TypeMask(InTypeMask)
{
}

FRHIDescriptorHandle FRHIHeapDescriptorAllocator::Allocate(ERHIDescriptorType InType)
{
	return FRHIDescriptorAllocator::Allocate(InType);
}

void FRHIHeapDescriptorAllocator::Free(TConstArrayView<FRHIDescriptorHandle> InHandles)
{
	if (InHandles.Num() > 0)
	{
#if DO_GUARD_SLOW
		for (const FRHIDescriptorHandle& Handle : InHandles)
		{
			checkSlow(HandlesAllocation(Handle.GetType()));
		}
#endif

		FRHIDescriptorAllocator::Free(InHandles);
	}
}

void FRHIHeapDescriptorAllocator::Free(FRHIDescriptorHandle InHandle)
{
	if (InHandle.IsValid())
	{
		check(HandlesAllocation(InHandle.GetType()));
		FRHIDescriptorAllocator::Free(InHandle);
	}
}

TOptional<FRHIDescriptorAllocation> FRHIHeapDescriptorAllocator::Allocate(uint32 NumDescriptors)
{
	return FRHIDescriptorAllocator::Allocate(NumDescriptors);
}

void FRHIHeapDescriptorAllocator::Free(FRHIDescriptorAllocation Allocation)
{
	FRHIDescriptorAllocator::Free(Allocation);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FRHIOffsetHeapDescriptorAllocator

FRHIOffsetHeapDescriptorAllocator::FRHIOffsetHeapDescriptorAllocator(ERHIDescriptorTypeMask InType, uint32 InDescriptorCount, uint32 InHeapOffset, TConstArrayView<TStatId> InStats)
	: FRHIHeapDescriptorAllocator(InType, InDescriptorCount, InStats)
	, HeapOffset(InHeapOffset)
{
}

FRHIDescriptorHandle FRHIOffsetHeapDescriptorAllocator::Allocate(ERHIDescriptorType InType)
{
	const FRHIDescriptorHandle AlocatorHandle = FRHIHeapDescriptorAllocator::Allocate(InType);
	if (AlocatorHandle.IsValid())
	{
		return FRHIDescriptorHandle(AlocatorHandle.GetType(), AlocatorHandle.GetIndex() + HeapOffset);
	}
	return FRHIDescriptorHandle();
}

void FRHIOffsetHeapDescriptorAllocator::Free(const FRHIDescriptorHandle InHandle)
{
	if (InHandle.IsValid())
	{
		const FRHIDescriptorHandle AdjustedHandle(InHandle.GetType(), InHandle.GetIndex() - HeapOffset);
		FRHIHeapDescriptorAllocator::Free(AdjustedHandle);
	}
}

TOptional<FRHIDescriptorAllocation> FRHIOffsetHeapDescriptorAllocator::Allocate(uint32 NumDescriptors)
{
	TOptional<FRHIDescriptorAllocation> Allocation = FRHIHeapDescriptorAllocator::Allocate(NumDescriptors);
	if (Allocation)
	{
		Allocation->StartIndex += HeapOffset;
	}
	return Allocation;
}

void FRHIOffsetHeapDescriptorAllocator::Free(FRHIDescriptorAllocation Allocation)
{
	Allocation.StartIndex -= HeapOffset;
	FRHIHeapDescriptorAllocator::Free(Allocation);
}

