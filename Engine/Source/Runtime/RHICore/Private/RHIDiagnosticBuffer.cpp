// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIDiagnosticBuffer.h"
#include "RHICoreShader.h"

FString FRHIDiagnosticBuffer::GetShaderDiagnosticMessages(uint32 DeviceIndex, uint32 QueueIndex, const TCHAR* QueueName)
{
	FString ShaderDiagnostics = FString::Printf(TEXT("\r\n\r\n\tDevice: %d, Queue %s:"), DeviceIndex, QueueName);

	if (!Data)
	{
		ShaderDiagnostics += TEXT("\r\n\t\tDiagnostic buffer not allocated for this queue.");
		return ShaderDiagnostics;
	}

	bool bFound = false;
	FString LanesString;
	for (int32 LaneIndex = 0; LaneIndex < UE_ARRAY_COUNT(Data->Lanes); ++LaneIndex)
	{
		FLane& Lane = Data->Lanes[LaneIndex];

		LanesString += FString::Printf(TEXT("\r\n\t\tLane %02d: "), LaneIndex);

		if (Lane.Counter)
		{
			bFound = true;

			const uint32 Line = Lane.Payload.AsUint[0];
			const FString* File = UE::RHICore::GetDiagnosticMessage(Lane.Payload.AsUint[1]);
			const FString* Message = UE::RHICore::GetDiagnosticMessage(Lane.Payload.AsUint[2]);

			if (File && Message)
			{
				LanesString += FString::Printf(TEXT("Shader assertion failed - %s:%d - %s"), **File, Line, **Message);
			}
			else
			{
				LanesString += FString::Printf(TEXT("Shader assertion failed - ID: 0x%08X (%d)"), Lane.MessageID, Lane.MessageID);

				{
					const int32* Payload = Lane.Payload.AsInt;
					if (Payload[0] < 0 || Payload[1] < 0 || Payload[2] < 0 || Payload[3] < 0)
					{
						LanesString += FString::Printf(TEXT("\r\n\t\t\tPayload [ int32]: %d %d %d %d"), Payload[0], Payload[1], Payload[2], Payload[3]);
					}
				}

				{
					const uint32* Payload = Lane.Payload.AsUint;
					LanesString += FString::Printf(TEXT("\r\n\t\t\tPayload [uint32]: %u %u %u %u"), Payload[0], Payload[1], Payload[2], Payload[3]);
				}

				{
					const uint32* Payload = Lane.Payload.AsUint;
					LanesString += FString::Printf(TEXT("\r\n\t\t\tPayload [   hex]: 0x%08X 0x%08X 0x%08X 0x%08X"), Payload[0], Payload[1], Payload[2], Payload[3]);
				}

				{
					const float* Payload = Lane.Payload.AsFloat;
					LanesString += FString::Printf(TEXT("\r\n\t\t\tPayload [ float]: %f %f %f %f"), Payload[0], Payload[1], Payload[2], Payload[3]);
				}
			}
		}
		else
		{
			LanesString += TEXT("OK");
		}
	}

	if (bFound)
	{
		ShaderDiagnostics += LanesString;
	}
	else
	{
		ShaderDiagnostics += TEXT("\r\n\t\tNo shader diagnostics found for this queue.");
	}

	return ShaderDiagnostics;
}
