// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIPoolAllocator.h"
#include "HAL/IConsoleManager.h"
#include "Async/Async.h"

static float GRHIPoolAllocatorDefragSizeFraction = 0.9f;
static FAutoConsoleVariableRef CVarRHIPoolAllocatorDefragSizeFraction(
	TEXT("RHIPoolAllocator.DefragSizeFraction"),
	GRHIPoolAllocatorDefragSizeFraction,
	TEXT("Skip defrag of pool if usage is more than given fraction (default: 0.9f)."),
	ECVF_Default);

static int32 GRHIPoolAllocatorDefragMaxPoolsToClear = 1;
static FAutoConsoleVariableRef CVarRHIPoolAllocatorDefragMaxPoolsToClear(
	TEXT("RHIPoolAllocator.DefragMaxPoolsToClear"),
	GRHIPoolAllocatorDefragMaxPoolsToClear,
	TEXT("Maximum amount of pools to try and clear during a single alloctor defrag call (default: 1 - negative then all pools will be tried and no timeslicing)"),
	ECVF_Default);

static int32 GRHIPoolAllocatorValidateLinkList = 0;
static FAutoConsoleVariableRef CVarRHIPoolAllocatorValidateLinkList(
	TEXT("RHIPoolAllocator.ValidateLinkedList"),
	GRHIPoolAllocatorValidateLinkList,
	TEXT("Validate all the internal linked list data of all the RHIPoolAllocators during every operation."),
	ECVF_Default);

//-----------------------------------------------------------------------------
//	LinkedList Allocator PrivateData
//-----------------------------------------------------------------------------

void FRHIPoolAllocationData::Reset()
{
	Size = 0;
	Alignment = 0;
	SetAllocationType(EAllocationType::Unknown);
	PoolIndex = -1;
	Offset = 0;
	Locked = false;

	Owner = nullptr;
	PreviousAllocation = nullptr;
	NextAllocation = nullptr;
	AliasAllocation = nullptr;
}

void FRHIPoolAllocationData::InitAsHead(int16 InPoolIndex)
{
	Reset();

	SetAllocationType(EAllocationType::Head);
	NextAllocation = this;
	PreviousAllocation = this;
	PoolIndex = InPoolIndex;
}

void FRHIPoolAllocationData::InitAsFree(int16 InPoolIndex, uint32 InSize, uint32 InAlignment, uint32 InOffset)
{
	Reset();

	Size = InSize;
	Alignment = InAlignment;
	SetAllocationType(EAllocationType::Free);
	Offset = InOffset;
	PoolIndex = InPoolIndex;
}


void FRHIPoolAllocationData::InitAsAllocated(uint32 InSize, uint32 InPoolAlignment, uint32 InAllocationAlignment, FRHIPoolAllocationData* InFree)
{
	check(InFree->IsFree());

	Reset();
	
	Size = InSize;
	Alignment = InAllocationAlignment;
	SetAllocationType(EAllocationType::Allocated);
	Offset = FRHIMemoryPool::GetAlignedOffset(InFree->Offset, InPoolAlignment, InAllocationAlignment);
	PoolIndex = InFree->PoolIndex;
	Locked = true;

	uint32 AlignedSize = FRHIMemoryPool::GetAlignedSize(InSize, InPoolAlignment, InAllocationAlignment);
	InFree->Size -= AlignedSize;
	InFree->Offset += AlignedSize;
	InFree->AddBefore(this);
}


void FRHIPoolAllocationData::MoveFrom(FRHIPoolAllocationData& InAllocated, bool InLocked)
{
	check(InAllocated.IsAllocated());
	
	Reset();

	Size = InAllocated.Size;
	Alignment = InAllocated.Alignment;
	Type = InAllocated.Type;
	Offset = InAllocated.Offset;
	PoolIndex = InAllocated.PoolIndex;
	Locked = InLocked;
	
	// Update linked list
	if (InAllocated.PreviousAllocation)
	{
		InAllocated.PreviousAllocation->NextAllocation = this;
		InAllocated.NextAllocation->PreviousAllocation = this;
		PreviousAllocation = InAllocated.PreviousAllocation;
		NextAllocation = InAllocated.NextAllocation;

		// Update aliases (if present) to point to new parent
		for (FRHIPoolAllocationData* Alias = InAllocated.GetFirstAlias(); Alias; Alias = Alias->GetNext())
		{
			Alias->AliasAllocation = this;
		}
		AliasAllocation = InAllocated.AliasAllocation;
	}
	else
	{
		// If there is no linked list, it means we are being passed an alias to a resource referenced by multiple GPUs.
		// For this case, we need to remove "InAllocated" from its original parent, and replace it with "this".  This is
		// a rare edge case, used in FD3D12DynamicRHI::UnlockBuffer in D3D12Buffer.cpp, where we are uploading a
		// single CPU staging buffer to multiple GPU mirror copies.
		FRHIPoolAllocationData* AliasParent = InAllocated.AliasAllocation;		// Get original parent of "InAllocated"
		check(AliasParent);

		InAllocated.RemoveAlias();				// Remove "InAllocated" as alias from original parent
		AliasParent->AddAlias(this);			// Add "this" as alias to original parent
	}

	InAllocated.Reset();
}


void FRHIPoolAllocationData::MarkFree(uint32 InPoolAlignment, uint32 InAllocationAlignment)
{
	check(IsAllocated());

	// Mark as free and update the size, offset and alignment according to the new requested alignment
	SetAllocationType(EAllocationType::Free);
	Size = FRHIMemoryPool::GetAlignedSize(Size, InPoolAlignment, InAllocationAlignment);
	Offset = AlignDown(Offset, InPoolAlignment);
	Alignment = InPoolAlignment;
}


void FRHIPoolAllocationData::Merge(FRHIPoolAllocationData* InOther)
{
	check(IsFree() && InOther->IsFree());
	check((Offset + Size) == InOther->Offset);
	check(PoolIndex == InOther->GetPoolIndex())

	Size += InOther->Size;

	InOther->RemoveFromLinkedList();
	InOther->Reset();
}


void FRHIPoolAllocationData::RemoveFromLinkedList()
{
	check(IsFree());
	PreviousAllocation->NextAllocation = NextAllocation;
	NextAllocation->PreviousAllocation = PreviousAllocation;
}


void FRHIPoolAllocationData::AddBefore(FRHIPoolAllocationData* InOther)
{
	PreviousAllocation->NextAllocation = InOther;
	InOther->PreviousAllocation = PreviousAllocation;

	PreviousAllocation = InOther;
	InOther->NextAllocation = this;
}


void FRHIPoolAllocationData::AddAfter(FRHIPoolAllocationData* InOther)
{
	NextAllocation->PreviousAllocation = InOther;
	InOther->NextAllocation = NextAllocation;

	NextAllocation = InOther;
	InOther->PreviousAllocation = this;
}

// The "AliasAllocation" in the original allocation points to a linked list of aliases.  The aliases
// are doubly linked with each other via NextAllocation/PreviousAllocation, and also point back to their
// list head via "AliasAllocation".
void FRHIPoolAllocationData::AddAlias(FRHIPoolAllocationData* InAlias)
{
	if (AliasAllocation)
	{
		AliasAllocation->PreviousAllocation = InAlias;
	}

	InAlias->NextAllocation = AliasAllocation;		// Add link to previous alias linked list head
	InAlias->PreviousAllocation = nullptr;			// New item is at the front, so it doesn't have a predecessor
	InAlias->AliasAllocation = this;				// Point to the original allocation that contains the aliases

	AliasAllocation = InAlias;						// Finally, update current alias linked list head
}

void FRHIPoolAllocationData::RemoveAlias()
{
	if (AliasAllocation)
	{
		if (PreviousAllocation)
		{
			PreviousAllocation->NextAllocation = NextAllocation;
		}
		else
		{
			// Removing item at list head
			AliasAllocation->AliasAllocation = NextAllocation;
		}
		if (NextAllocation)
		{
			NextAllocation->PreviousAllocation = PreviousAllocation;
		}

		NextAllocation = nullptr;
		PreviousAllocation = nullptr;
		AliasAllocation = nullptr;
	}
}


/**
 * Sort predicate that sorts on size of the allocation data
 */
static bool LinkedListSortBySizePredicate(const FRHIPoolAllocationData* InLeft, const FRHIPoolAllocationData* InRight)
{
	return InLeft->GetSize() < InRight->GetSize();
}


/**
 * Sort predicate that sorts on offset of the allocation data
 */
static bool LinkedListSortByOffsetPredicate(const FRHIPoolAllocationData* InLeft, const FRHIPoolAllocationData* InRight)
{
	return InLeft->GetOffset() < InRight->GetOffset();
}


//-----------------------------------------------------------------------------
//	LinkedList Allocator
//-----------------------------------------------------------------------------

static int32 DesiredAllocationPoolSize = 32;

uint32 FRHIMemoryPool::GetAlignedSize(uint32 InSizeInBytes, uint32 InPoolAlignment, uint32 InAllocationAlignment)
{
	check(InAllocationAlignment <= InPoolAlignment);

	// Compute the size, taking the pool and allocation alignmend into account (conservative size)
	uint32 Size = ((InPoolAlignment % InAllocationAlignment) != 0) ? InSizeInBytes + InAllocationAlignment : InSizeInBytes;
	return AlignArbitrary(Size, InPoolAlignment);
}


uint32 FRHIMemoryPool::GetAlignedOffset(uint32 InOffset, uint32 InPoolAlignment, uint32 InAllocationAlignment)
{
	uint32 AlignedOffset = InOffset;

	// fix the offset with the requested alignment if needed
	if ((InPoolAlignment % InAllocationAlignment) != 0)
	{
		uint32 AlignmentRest = AlignedOffset % InAllocationAlignment;
		if (AlignmentRest > 0)
		{
			AlignedOffset += (InAllocationAlignment - AlignmentRest);
		}
	}

	return AlignedOffset;
}


FRHIMemoryPool::FRHIMemoryPool(int16 InPoolIndex, uint64 InPoolSize, uint32 InPoolAlignment, ERHIPoolResourceTypes InSupportedResourceTypes, EFreeListOrder InFreeListOrder) :
	PoolIndex(InPoolIndex), 
	PoolSize(InPoolSize), 
	PoolAlignment(InPoolAlignment),
	SupportedResourceTypes(InSupportedResourceTypes),
	FreeListOrder(InFreeListOrder),
	FreeSize(0), 
	AligmnentWaste(0), 
	AllocatedBlocks(0)
{	
	// PoolSize should fit in 32 bits because FRHIPoolAllocationData only stores 32bit size
	check(PoolSize < UINT32_MAX);
}


FRHIMemoryPool::~FRHIMemoryPool()
{
	// Can't currently validate because Engine is still leaking too much during shutdown sadly enough
	/*
	check(AllocatedBlocks == 0);
	check(AligmnentWaste == 0);
	check(FreeSize == PoolSize);
	check(FreeBlocks.Num() == 1);
	check(FreeBlocks[0]->GetSize() == PoolSize);
	*/

	Validate();
	Destroy();
}


void FRHIMemoryPool::Init()
{
	FreeSize = PoolSize;

	// Setup a few working free blocks	
	AllocationDataPool.Reserve(DesiredAllocationPoolSize);
	for (int32 Index = 0; Index < DesiredAllocationPoolSize; ++Index)
	{
		FRHIPoolAllocationData* AllocationData = new FRHIPoolAllocationData();
		AllocationData->Reset();
		AllocationDataPool.Add(AllocationData);
	}

	// Async allocated pool will not have a proper index at Init time
	if (PoolIndex != InvalidPoolIndex)
	{
		InitFreeBlocks();
	}
}

void FRHIMemoryPool::InitFreeBlocks()
{
	// Setup the special start and free block
	check(PoolIndex != InvalidPoolIndex);
	HeadBlock.InitAsHead(PoolIndex);
	FRHIPoolAllocationData* FreeBlock = GetNewAllocationData();
	FreeBlock->InitAsFree(PoolIndex, PoolSize, PoolAlignment, 0);
	HeadBlock.AddAfter(FreeBlock);
	FreeBlocks.Add(FreeBlock);

	Validate();
}


void FRHIMemoryPool::Destroy()
{
	// Validate state?

	for (FRHIPoolAllocationData* FreeBlock : FreeBlocks)
	{
		ReleaseAllocationData(FreeBlock);
	}
	FreeBlocks.Empty();
	
	for (FRHIPoolAllocationData* AllocationData : AllocationDataPool)
	{
		delete AllocationData;
	}
	AllocationDataPool.Empty();
}


bool FRHIMemoryPool::TryAllocate(uint32 InSizeInBytes, uint32 InAllocationAlignment, ERHIPoolResourceTypes InAllocationResourceType, FRHIPoolAllocationData& AllocationData)
{
	check(IsResourceTypeSupported(InAllocationResourceType));

	int32 FreeBlockIndex = FindFreeBlock(InSizeInBytes, InAllocationAlignment);
	if (FreeBlockIndex != INDEX_NONE)
	{
		uint32 AlignedSize = GetAlignedSize(InSizeInBytes, PoolAlignment, InAllocationAlignment);

		FRHIPoolAllocationData* FreeBlock = FreeBlocks[FreeBlockIndex];
		check(FreeBlock->GetSize() >= AlignedSize);

		// Remove from the free blocks because size will change and need sorted insert again then
		FreeBlocks.RemoveAt(FreeBlockIndex);

		// update private allocator data of new and free block
		AllocationData.InitAsAllocated(InSizeInBytes, PoolAlignment, InAllocationAlignment, FreeBlock);
		check((AllocationData.GetOffset() % InAllocationAlignment) == 0);

		// Update working stats
		check(FreeSize >= AlignedSize);
		FreeSize -= AlignedSize;
		AligmnentWaste += (AlignedSize - InSizeInBytes);
		AllocatedBlocks++;

		// Free block is empty then release otherwise sorted reinsert 
		if (FreeBlock->GetSize() == 0)
		{
			FreeBlock->RemoveFromLinkedList();
			ReleaseAllocationData(FreeBlock);

			Validate();
		}
		else
		{
			AddToFreeBlocks(FreeBlock);
		}
		
		return true;
	}
	else
	{
		return false;
	}
}


void FRHIMemoryPool::Deallocate(FRHIPoolAllocationData& AllocationData)
{
	check(AllocationData.IsLocked());
	check(PoolIndex == AllocationData.GetPoolIndex());

	// Original allocation is used to compute the orignal aligned allocation size
	uint32 AllocationAlignment = AllocationData.GetAlignment();

	// Free block should not be locked anymore - can be reused immediatly when we get to actual pool deallocate
	bool bLocked = false;
	uint64 AllocationSize = AllocationData.GetSize();

	FRHIPoolAllocationData* FreeBlock = GetNewAllocationData();
	FreeBlock->MoveFrom(AllocationData, bLocked);
	FreeBlock->MarkFree(PoolAlignment, AllocationAlignment);
		
	// Update working stats
	FreeSize += FreeBlock->GetSize();
	AligmnentWaste -= FreeBlock->GetSize() - AllocationSize;
	AllocatedBlocks--;

	AddToFreeBlocks(FreeBlock);
}


void FRHIMemoryPool::TryClear(FRHIContextArray const& Contexts, FRHIPoolAllocator* InAllocator, uint32 InMaxCopySize, uint32& CopySize, const TArray<FRHIMemoryPool*>& InTargetPools)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRHIMemoryPool::TryClear);

	FRHIPoolAllocationData* BlockToMove = HeadBlock.GetPrev();
	while (BlockToMove != &HeadBlock && CopySize < InMaxCopySize)
	{
		FRHIPoolAllocationData* NextBlockToMove = BlockToMove->GetPrev();

		// Only try to move if allocated (not free) and not 'locked' by another operation
		if (BlockToMove->IsAllocated() && !BlockToMove->IsLocked() && BlockToMove->GetOwner())
		{
			uint64 SizeToAllocate = BlockToMove->GetSize();
			uint32 AllocationAlignment = BlockToMove->GetAlignment();

			FRHIPoolAllocationData TempTargetAllocation;

			// Try find pool to move requested allocation (in desired order already)
			for (FRHIMemoryPool* TargetPool : InTargetPools)
			{
				if (TargetPool->TryAllocate(SizeToAllocate, AllocationAlignment, SupportedResourceTypes, TempTargetAllocation))
				{
					// RHI specific handling of the actual defrag request
					InAllocator->HandleDefragRequest(Contexts, BlockToMove, TempTargetAllocation);

					// Increment the working copy size
					CopySize += BlockToMove->GetSize();

					break;
				}
			}
		}

		BlockToMove = NextBlockToMove;
	}

	Validate();	
}


int32 FRHIMemoryPool::FindFreeBlock(uint32 InSizeInBytes, uint32 InAllocationAlignment) const
{
	uint32 AlignedSize = GetAlignedSize(InSizeInBytes, PoolAlignment, InAllocationAlignment);

	// Early out if total free size doesn't fit
	if (FreeSize < AlignedSize)
	{
		return INDEX_NONE;
	}

	if (FreeListOrder == EFreeListOrder::SortBySize)
	{
		// check first index as fast path
		if (FreeBlocks[0]->GetSize() >= AlignedSize)
		{
			return 0;
		}

		int32 FindIndex = Algo::LowerBoundBy(FreeBlocks, AlignedSize, [](const FRHIPoolAllocationData* FreeBlock)
			{
				return FreeBlock->GetSize();
			});
		if (FindIndex < FreeBlocks.Num())
		{
			return FindIndex;
		}
	}
	else
	{
		// Check all the free blocks (sorted by size, so take best fit)
		for (int32 FreeBlockIndex = 0; FreeBlockIndex < FreeBlocks.Num(); ++FreeBlockIndex)
		{
			if (FreeBlocks[FreeBlockIndex]->GetSize() >= AlignedSize)
			{
				return FreeBlockIndex;
			}
		}
	}

	return INDEX_NONE;
}


void FRHIMemoryPool::RemoveFromFreeBlocks(FRHIPoolAllocationData* InFreeBlock)
{
	for (int32 FreeBlockIndex = 0; FreeBlockIndex < FreeBlocks.Num(); ++FreeBlockIndex)
	{
		if (FreeBlocks[FreeBlockIndex] == InFreeBlock)
		{
			FreeBlocks.RemoveAt(FreeBlockIndex, EAllowShrinking::No);
			break;
		}
	}	
}


FRHIPoolAllocationData* FRHIMemoryPool::AddToFreeBlocks(FRHIPoolAllocationData* InFreeBlock)
{
	check(InFreeBlock->IsFree());
	check(IsAligned(InFreeBlock->GetSize(), PoolAlignment));
	check(IsAligned(InFreeBlock->GetOffset(), PoolAlignment));

	FRHIPoolAllocationData* FreeBlock = InFreeBlock;

	// Coalesce with previous?
	if (FreeBlock->GetPrev()->IsFree() && !FreeBlock->GetPrev()->IsLocked())
	{
		FRHIPoolAllocationData* PrevFree = FreeBlock->GetPrev();
		PrevFree->Merge(FreeBlock);
		RemoveFromFreeBlocks(PrevFree);
		ReleaseAllocationData(FreeBlock);

		FreeBlock = PrevFree;
	}

	// Coalesce with next?
	if (FreeBlock->GetNext()->IsFree() && !FreeBlock->GetNext()->IsLocked())
	{
		FRHIPoolAllocationData* NextFree = FreeBlock->GetNext();
		FreeBlock->Merge(NextFree);
		RemoveFromFreeBlocks(NextFree);
		ReleaseAllocationData(NextFree);
	}

	// Sorted insert
	int32 InsertIndex = 0;
	switch (FreeListOrder)
	{
		case EFreeListOrder::SortBySize:
		{
			InsertIndex = Algo::LowerBound(FreeBlocks, FreeBlock, LinkedListSortBySizePredicate);
			break;
		}
		case EFreeListOrder::SortByOffset:
		{
			InsertIndex = Algo::LowerBound(FreeBlocks, FreeBlock, LinkedListSortByOffsetPredicate);
			break;
		}
	}	
	FreeBlocks.Insert(FreeBlock, InsertIndex);

	Validate();

	return FreeBlock;
}


FRHIPoolAllocationData* FRHIMemoryPool::GetNewAllocationData()
{
	return (AllocationDataPool.Num() > 0) ? AllocationDataPool.Pop(EAllowShrinking::No) : new FRHIPoolAllocationData();
}


void FRHIMemoryPool::ReleaseAllocationData(FRHIPoolAllocationData* InData)
{
	InData->Reset();
	if (AllocationDataPool.Num() >= DesiredAllocationPoolSize)
	{
		delete InData;
	}
	else
	{
		AllocationDataPool.Add(InData);
	}
}


void FRHIMemoryPool::Validate()
{
	if (GRHIPoolAllocatorValidateLinkList <= 0)
		return;

	if (PoolIndex == InvalidPoolIndex)
		return;

	uint64 TotalFreeSize = 0;
	uint64 TotalAllocatedSize = 0;
	uint64 CurrentOffset = 0;

	// Validate the linked list
	FRHIPoolAllocationData* CurrentBlock = HeadBlock.GetNext();
	while (CurrentBlock != &HeadBlock)
	{
		// validate linked list
		check(CurrentBlock == CurrentBlock->GetNext()->GetPrev());
		check(CurrentBlock == CurrentBlock->GetPrev()->GetNext());

		uint32 AlignedSize = GetAlignedSize(CurrentBlock->GetSize(), PoolAlignment, CurrentBlock->GetAlignment());
		uint32 AlignedOffset = AlignDown(CurrentBlock->GetOffset(), PoolAlignment);

		check(AlignedOffset == CurrentOffset);

		if (CurrentBlock->IsAllocated())
		{
			TotalAllocatedSize += AlignedSize;
		}
		else if (CurrentBlock->IsFree())
		{
			TotalFreeSize += AlignedSize;
		}
		else
		{
			check(false);
		}

		CurrentOffset += AlignedSize;
		CurrentBlock = CurrentBlock->GetNext();
	}

	check(TotalFreeSize + TotalAllocatedSize == PoolSize);
	check(TotalFreeSize == FreeSize);
	check(TotalAllocatedSize == GetUsedSize());

	// Validate the free blocks
	TotalFreeSize = 0;
	for (int32 FreeBlockIndex = 0; FreeBlockIndex < FreeBlocks.Num(); ++FreeBlockIndex)
	{
		FRHIPoolAllocationData* FreeBlock = FreeBlocks[FreeBlockIndex];
		check(!FreeBlock->GetPrev()->IsFree() || FreeBlock->GetPrev()->IsLocked());
		check(!FreeBlock->GetNext()->IsFree() || FreeBlock->GetNext()->IsLocked());
		check(FreeListOrder != EFreeListOrder::SortBySize || FreeBlockIndex == (FreeBlocks.Num() - 1) || FreeBlock->GetSize() <= FreeBlocks[FreeBlockIndex + 1]->GetSize());
		check(IsAligned(FreeBlock->GetOffset(), PoolAlignment));
		TotalFreeSize += FreeBlock->GetSize();
	}
	check(TotalFreeSize == FreeSize);
}


//-----------------------------------------------------------------------------
//	Pool Allocator
//-----------------------------------------------------------------------------

FRHIPoolAllocator::FRHIPoolAllocator(uint64 InDefaultPoolSize, uint32 InPoolAlignment, uint32 InMaxAllocationSize, FRHIMemoryPool::EFreeListOrder InFreeListOrder, 
	bool InDefragEnabled, ERHIPoolAllocationStrategy InPoolAllocationStrategy, uint16 InOverFlowPoolsCount):
	DefaultPoolSize(InDefaultPoolSize),
	PoolAlignment(InPoolAlignment),
	MaxAllocationSize(InMaxAllocationSize),
	FreeListOrder(InFreeListOrder),
	bDefragEnabled(InDefragEnabled),
	LastDefragPoolIndex(0),
	TotalAllocatedBlocks(0),
	PoolAllocationStrategy(InPoolAllocationStrategy),
	OverFlowPoolsCount(InOverFlowPoolsCount)
{}


FRHIPoolAllocator::~FRHIPoolAllocator()
{
	Destroy();
}


void FRHIPoolAllocator::Initialize(ERHIPoolResourceTypes InAsyncAllocationResourceType)
{
	if (PoolAllocationStrategy == ERHIPoolAllocationStrategy::Async)
	{
		check(Pools.Num() == 0);

		Pools.Add(CreateNewPool(0, 16, InAsyncAllocationResourceType));
		PoolAllocationOrder.Add(0);

		check(OverFlowPoolsCount > 0);

		PoolAllocationStatus = ERHIPoolAllocationStatus::Allocating;

		AsyncTask(ENamedThreads::AnyBackgroundHiPriTask, [this, InAsyncAllocationResourceType]()
			{
				PoolAsyncAllocate(DefaultPoolSize, 16, InAsyncAllocationResourceType);
			});
	}
}


void FRHIPoolAllocator::Destroy()
{
	if (PoolAllocationStrategy == ERHIPoolAllocationStrategy::Async)
	{
		FScopeLock Lock(&CS);
		PoolAllocationStatus = ERHIPoolAllocationStatus::ShutDown;
	}

	for (int32 i = (Pools.Num() - 1); i >= 0; i--)
	{
		if (Pools[i] != nullptr)
		{
			Pools[i]->Destroy();
			delete(Pools[i]);
		}
	}

	for (int32 c = 0; c < OverFlowPools.Num(); c++)
	{
		if (OverFlowPools[c] != nullptr)
		{
			OverFlowPools[c]->Destroy();
			delete(OverFlowPools[c]);
		}
	}
	
	Pools.Empty();
	PoolAllocationOrder.Empty();
	OverFlowPools.Empty();
}


bool FRHIPoolAllocator::TryAllocateInternal(uint32 InSizeInBytes, uint32 InAllocationAlignment, ERHIPoolResourceTypes InAllocationResourceType, FRHIPoolAllocationData& AllocationData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRHIPoolAllocator::TryAllocateInternal);

	if (PoolAllocationStrategy == ERHIPoolAllocationStrategy::Async)
	{
		CS.Lock();

		check(PoolAllocationOrder.Num() == Pools.Num())

		bool bResourceAllocated = false;

		for (int32 PoolIndex : PoolAllocationOrder)
		{
			FRHIMemoryPool* Pool = Pools[PoolIndex];
			if (Pool != nullptr && Pool->IsResourceTypeSupported(InAllocationResourceType) && !Pool->IsFull() && Pool->TryAllocate(InSizeInBytes, InAllocationAlignment, InAllocationResourceType, AllocationData))
			{
				bResourceAllocated = true;
				break;
			}
		}
		uint32 AlignedSize = 0;
		// No pool available, grab an overflow pool or we need to allocate a new pool right now
		if (!bResourceAllocated)
		{ 
			// Find a free pool index
			int16 NewPoolIndex = -1;
			for (int32 PoolIndex = 0; PoolIndex < Pools.Num(); PoolIndex++)
			{
				if (Pools[PoolIndex] == nullptr)
				{
					NewPoolIndex = PoolIndex;
					break;
				}
			}

			int16 PoolIndexForInit = (NewPoolIndex >= 0) ? NewPoolIndex : Pools.Num();

			AlignedSize = FRHIMemoryPool::GetAlignedSize(InSizeInBytes, PoolAlignment, InAllocationAlignment);

			FRHIMemoryPool* ReadyPool = nullptr;
			for (int32 OverFlowPoolIndex = 0; OverFlowPoolIndex < OverFlowPools.Num(); OverFlowPoolIndex++)
			{
				if ((OverFlowPools[OverFlowPoolIndex]->IsResourceTypeSupported(InAllocationResourceType)) 
					&& (OverFlowPools[OverFlowPoolIndex]->GetPoolAlignment() == PoolAlignment)
					&& (OverFlowPools[OverFlowPoolIndex]->GetPoolSize() >= AlignedSize))
				{
					ReadyPool = OverFlowPools[OverFlowPoolIndex];

					check(PoolIndexForInit != -1);
					
					// Pool needs correct PoolIndex for FreeBlocks
					ReadyPool->SetPoolIndex(PoolIndexForInit);
					ReadyPool->InitFreeBlocks();

					check(OverFlowPools[OverFlowPoolIndex]->IsEmpty());
					OverFlowPools.RemoveAtSwap(OverFlowPoolIndex);
					break;
				}
			}

			// No overflow pools ready, need to allocate inline a new pool
			if (ReadyPool == nullptr)
			{
				ReadyPool = CreateNewPool(PoolIndexForInit, AlignedSize, InAllocationResourceType);
			}

			if (NewPoolIndex >= 0)
			{
				Pools[NewPoolIndex] = ReadyPool;
			}
			else
			{
				NewPoolIndex = Pools.Num();
				Pools.Add(ReadyPool);
				PoolAllocationOrder.Add(NewPoolIndex);
			}

			check(NewPoolIndex == ReadyPool->GetPoolIndex());

			bResourceAllocated = Pools[NewPoolIndex]->TryAllocate(InSizeInBytes, InAllocationAlignment, InAllocationResourceType, AllocationData);
		}

		// Test if we have enough overflow pools available
		int32 FreePoolsCount = 0;

		for (int32 OverFlowPoolIndex = 0; OverFlowPoolIndex < OverFlowPools.Num(); OverFlowPoolIndex++)
		{
			if ((OverFlowPools[OverFlowPoolIndex]->IsResourceTypeSupported(InAllocationResourceType))
				&& (OverFlowPools[OverFlowPoolIndex]->GetPoolAlignment() == PoolAlignment)
				&& (OverFlowPools[OverFlowPoolIndex]->GetPoolSize() >= AlignedSize))
			{
				FreePoolsCount++;
			}
		}

		if ((FreePoolsCount < OverFlowPoolsCount) && (PoolAllocationStatus == ERHIPoolAllocationStatus::Idle))
		{
			PoolAllocationStatus = ERHIPoolAllocationStatus::Allocating;

			CS.Unlock();

			AsyncTask(ENamedThreads::AnyBackgroundHiPriTask, [this, /*DefaultPoolSize, PoolAlignment,*/ InAllocationResourceType]()
			{
				PoolAsyncAllocate(DefaultPoolSize, PoolAlignment, InAllocationResourceType);
			});
		}
		else
		{
			CS.Unlock();
		}

		return bResourceAllocated;
	}
	else
	{
		FScopeLock Lock(&CS);

		check(PoolAllocationOrder.Num() == Pools.Num())

		for (int32 PoolIndex : PoolAllocationOrder)
		{
			FRHIMemoryPool* Pool = Pools[PoolIndex];
			if (Pool != nullptr && Pool->IsResourceTypeSupported(InAllocationResourceType) && !Pool->IsFull() && Pool->TryAllocate(InSizeInBytes, InAllocationAlignment, InAllocationResourceType, AllocationData))
			{
				return true;
			}
		}

		// Find a free pool index
		int16 NewPoolIndex = -1;
		for (int32 PoolIndex = 0; PoolIndex < Pools.Num(); PoolIndex++)
		{
			if (Pools[PoolIndex] == nullptr)
			{
				NewPoolIndex = PoolIndex;
				break;
			}
		}

		uint32 AlignedSize = FRHIMemoryPool::GetAlignedSize(InSizeInBytes, PoolAlignment, InAllocationAlignment);
		if (NewPoolIndex >= 0)
		{
			Pools[NewPoolIndex] = CreateNewPool(NewPoolIndex, AlignedSize, InAllocationResourceType);
		}
		else
		{
			NewPoolIndex = Pools.Num();
			Pools.Add(CreateNewPool(NewPoolIndex, AlignedSize, InAllocationResourceType));
			PoolAllocationOrder.Add(NewPoolIndex);
		}

		return Pools[NewPoolIndex]->TryAllocate(InSizeInBytes, InAllocationAlignment, InAllocationResourceType, AllocationData);
	}
}

void FRHIPoolAllocator::PoolAsyncAllocate(uint32 InSizeInBytes, uint32 InAllocationAlignment, ERHIPoolResourceTypes InAllocationResourceType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRHIPoolAllocator::PoolAsyncAllocate);

	uint32 AlignedSize = FRHIMemoryPool::GetAlignedSize(InSizeInBytes, PoolAlignment, InAllocationAlignment);
	FRHIMemoryPool* NewPool = CreateNewPool(FRHIMemoryPool::InvalidPoolIndex, AlignedSize, InAllocationResourceType);

	{
		FScopeLock Lock(&CS);
		check(PoolAllocationStrategy == ERHIPoolAllocationStrategy::Async);
		
		if (PoolAllocationStatus == ERHIPoolAllocationStatus::Allocating)
		{
			// Allocate the pool without knowing the correct Index, it will be initialized later when used for allocations
			OverFlowPools.Add(NewPool);
		}
		else
		{
			delete NewPool;
		}

		PoolAllocationStatus = ERHIPoolAllocationStatus::Idle;
	}
}


void FRHIPoolAllocator::DeallocateInternal(FRHIPoolAllocationData& AllocationData)
{
	FScopeLock Lock(&CS);

	check(AllocationData.IsAllocated());
	check(AllocationData.GetPoolIndex() < Pools.Num());
	Pools[AllocationData.GetPoolIndex()]->Deallocate(AllocationData);
}


void FRHIPoolAllocator::SortPools()
{
	Algo::Sort(PoolAllocationOrder, [this](uint32 InLHS, uint32 InRHS)
		{
			// first allocate from 'fullest' pools when defrag is enabled, otherwise try and allocate from
			// default sized pools with stable order
			if (bDefragEnabled)
			{
				uint32 LHSUsePoolSize = Pools[InLHS] ? Pools[InLHS]->GetUsedSize() : UINT32_MAX;
				uint32 RHSUsePoolSize = Pools[InRHS] ? Pools[InRHS]->GetUsedSize() : UINT32_MAX;
				return LHSUsePoolSize > RHSUsePoolSize;
			}
			else
			{
				uint32 LHSPoolSize = Pools[InLHS] ? Pools[InLHS]->GetPoolSize() : UINT32_MAX;
				uint32 RHSPoolSize = Pools[InRHS] ? Pools[InRHS]->GetPoolSize() : UINT32_MAX;
				if (LHSPoolSize != RHSPoolSize)
				{
					return LHSPoolSize < RHSPoolSize;
				}

				uint32 LHSPoolIndex = Pools[InLHS] ? Pools[InLHS]->GetPoolIndex() : UINT32_MAX;
				uint32 RHSPoolIndex = Pools[InRHS] ? Pools[InRHS]->GetPoolIndex() : UINT32_MAX;
				return LHSPoolIndex < RHSPoolIndex;
			}
		});
}


void FRHIPoolAllocator::Defrag(FRHIContextArray const& Contexts, uint32 InMaxCopySize, uint32& CurrentCopySize)
{
	// Don't do anything when defrag is disabled for this allocator
	if (!bDefragEnabled)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FRHIPoolAllocator::Defrag);

	FScopeLock Lock(&CS);

	TArray<FRHIMemoryPool*> SortedTargetPools;
	SortedTargetPools.Reserve(Pools.Num());
	uint32 MaxPoolsToDefrag = 0;
	for (FRHIMemoryPool* Pool : Pools)
	{
		// collect all non full pools as target for defrag
		if (Pool && !Pool->IsFull())
		{
			SortedTargetPools.Add(Pool);
		}

		// compute the usage percentage of the pool - don't defrag if used more than certain value
		float Usage = Pool ? (float)Pool->GetUsedSize() / (float)Pool->GetPoolSize() : 1.0f;
		if (Usage < GRHIPoolAllocatorDefragSizeFraction)
		{
			MaxPoolsToDefrag++;
		}
	}

	// Need more than 1 pool at least
	if (MaxPoolsToDefrag == 0 || SortedTargetPools.Num() < 2)
	{
		return;
	}

	Algo::Sort(SortedTargetPools, [](const FRHIMemoryPool* InLHS, const FRHIMemoryPool* InRHS)
		{
			return InLHS->GetUsedSize() < InRHS->GetUsedSize();
		});

	TArray<FRHIMemoryPool*> TargetPools;
	if (GRHIPoolAllocatorDefragMaxPoolsToClear < 0)
	{
		// Build list all target pools (added reverse order because we want to move to the fullest pool first)
		// Skip index 0 because that's the pool we want to clear to the target pools
		TargetPools.Reserve(SortedTargetPools.Num());
		for (int32 PoolIndex = SortedTargetPools.Num() - 1; PoolIndex > 0; --PoolIndex)
		{
			TargetPools.Add(SortedTargetPools[PoolIndex]);
		}

		for (int32 PoolIndex = 0; PoolIndex < SortedTargetPools.Num() - 1; ++PoolIndex)
		{
			FRHIMemoryPool* PoolToClear = SortedTargetPools[PoolIndex];
			PoolToClear->TryClear(Contexts, this, InMaxCopySize, CurrentCopySize, TargetPools);

			// Remove last allocator since we will try and clear that one next		
			TargetPools.Pop();

			if (CurrentCopySize >= InMaxCopySize)
			{
				break;
			}
		}
	}
	else
	{
		if (LastDefragPoolIndex >= Pools.Num())
		{
			LastDefragPoolIndex = 0;
		}
		int32 DefraggedPoolCount = 0;
		for (; LastDefragPoolIndex < Pools.Num(); LastDefragPoolIndex++)
		{
			FRHIMemoryPool* PoolToClear = Pools[LastDefragPoolIndex];

			// skip if invalid, or too full
			float Usage = PoolToClear ? (float)PoolToClear->GetUsedSize() / (float)PoolToClear->GetPoolSize() : 1.0f;
			if (Usage >= GRHIPoolAllocatorDefragSizeFraction)
			{
				continue;
			}

			// Build list all target pools (added reverse order because we want to move to the fullest pool first)
			TargetPools.Empty(SortedTargetPools.Num());
			for (int32 SortedPoolIndex = SortedTargetPools.Num() - 1; SortedPoolIndex >= 0; --SortedPoolIndex)
			{
				FRHIMemoryPool* TargetPool = SortedTargetPools[SortedPoolIndex];
				if (PoolToClear == TargetPool)
				{
					break;
				}
				TargetPools.Add(TargetPool);
			}

			// still have target pools?
			if (TargetPools.Num() == 0)
			{
				continue;
			}

			PoolToClear->TryClear(Contexts, this, InMaxCopySize, CurrentCopySize, TargetPools);
			DefraggedPoolCount++;

			if (CurrentCopySize >= InMaxCopySize || (DefraggedPoolCount >= GRHIPoolAllocatorDefragMaxPoolsToClear))
			{
				break;
			}
		}
	}
}


void FRHIPoolAllocator::UpdateMemoryStats(uint32& IOMemoryAllocated, uint32& IOMemoryUsed, uint32& IOMemoryFree, uint32& IOMemoryEndFree, uint32& IOAlignmentWaste, uint32& IOAllocatedPageCount, uint32& IOFullPageCount)
{
	FScopeLock Lock(&CS);

	TotalAllocatedBlocks = 0;

	// found out which pool has maximum available free size
	uint32 MaxEndFree = 0;
	for (FRHIMemoryPool* Pool : Pools)
	{
		if (Pool)
		{
			IOMemoryAllocated += Pool->GetPoolSize();
			IOMemoryUsed += Pool->GetUsedSize() - Pool->GetAlignmentWaste();
			IOMemoryFree += Pool->GetFreeSize();
			MaxEndFree = FMath::Max(MaxEndFree, (uint32)Pool->GetFreeSize());
			IOAlignmentWaste += Pool->GetAlignmentWaste();
			IOAllocatedPageCount++;
			if (Pool->IsFull())
			{
				IOFullPageCount++;
			}

			TotalAllocatedBlocks += Pool->GetAllocatedBlocks();
		}
	}

	IOMemoryEndFree += MaxEndFree;
}
