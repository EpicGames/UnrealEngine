// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIGlobals.h"
#include "RenderResource.h"

/**
 * A list of the most recently used bound shader states.
 * This is used to keep bound shader states that have been used recently from being freed, as they're likely to be used again soon.
 */
template<uint32 Size, bool TThreadSafe = true>
class TBoundShaderStateHistory : public FRenderResource
{
public:
	TBoundShaderStateHistory() = default;

	/** Adds a bound shader state to the history. */
	inline void Add(FRHIBoundShaderState* BoundShaderState)
	{
		if (TThreadSafe && GRHISupportsParallelRHIExecute)
		{
			BoundShaderStateHistoryLock.Lock();
		}
		BoundShaderStates[NextBoundShaderStateIndex] = BoundShaderState;
		NextBoundShaderStateIndex = (NextBoundShaderStateIndex + 1) % Size;
		if (TThreadSafe && GRHISupportsParallelRHIExecute)
		{
			BoundShaderStateHistoryLock.Unlock();
		}
	}

	FRHIBoundShaderState* GetLast()
	{
		check(!GRHISupportsParallelRHIExecute);
		// % doesn't work as we want on negative numbers, so handle the wraparound manually
		uint32 LastIndex = NextBoundShaderStateIndex == 0 ? Size - 1 : NextBoundShaderStateIndex - 1;
		return BoundShaderStates[LastIndex];
	}

	// FRenderResource interface.
	virtual void ReleaseRHI()
	{
		if (TThreadSafe && GRHISupportsParallelRHIExecute)
		{
			BoundShaderStateHistoryLock.Lock();
		}
		for (uint32 Index = 0; Index < Size; Index++)
		{
			BoundShaderStates[Index].SafeRelease();
		}
		if (TThreadSafe && GRHISupportsParallelRHIExecute)
		{
			BoundShaderStateHistoryLock.Unlock();
		}
	}

private:
	FBoundShaderStateRHIRef BoundShaderStates[Size];
	uint32 NextBoundShaderStateIndex = 0;
	FCriticalSection BoundShaderStateHistoryLock;
};
