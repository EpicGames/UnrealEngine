// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "RHIResources.h"

RHICORE_API DECLARE_LOG_CATEGORY_EXTERN(LogRHICore, Log, VeryVerbose);

namespace UE::RHICore
{

struct FResolveTextureInfo
{
	FResolveTextureInfo() = default;
	
	FResolveTextureInfo(
		FRHITexture* InSourceTexture,
		FRHITexture* InDestTexture,
		uint8 InMipLevel,
		int32 InArraySlice,
		FResolveRect InResolveRect)
		: SourceTexture(InSourceTexture)
		, DestTexture(InDestTexture)
		, MipLevel(InMipLevel)
		, ArraySlice(InArraySlice)
	{}

	FRHITexture* SourceTexture = nullptr;
	FRHITexture* DestTexture = nullptr;
	uint8 MipLevel = 0;
	int32 ArraySlice = -1;
	FResolveRect ResolveRect;
};

RHICORE_API void ResolveRenderPassTargets(const FRHIRenderPassInfo& Info, TFunction<void(FResolveTextureInfo)> Function);
RHICORE_API FRHIViewDesc::EDimension AdjustViewInfoDimensionForNarrowing(const FRHIViewDesc::FTexture::FViewInfo& ViewInfo, const FRHITextureDesc& TextureDesc);

RHICORE_API bool AreRayTracingShadersEnabled();

RHICORE_API bool AllowVendorDevice();

// Utility for RHIs to load the bindless configuration before ShaderPlatformConfig is initialized.
RHICORE_API ERHIBindlessConfiguration GetBindlessConfigurationOnStartup(EShaderPlatform Platform);

} //! UE::RHICore

