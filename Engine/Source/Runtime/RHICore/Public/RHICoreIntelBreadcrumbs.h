// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef INTEL_GPU_CRASH_DUMPS
#	define INTEL_GPU_CRASH_DUMPS 0
#endif

#if INTEL_GPU_CRASH_DUMPS

	#include "RHIBreadcrumbs.h"

	namespace UE::RHICore::Intel::GPUCrashDumps
	{
		static bool bEnabled = false;
		static uint32 Flags = 0;

		// API
		RHICORE_API bool IsEnabled();

		//
		// Called on RHICore module startup to load the Intel Extensions library
		//
		RHICORE_API void StartupModule();

		//
		// Called by platform RHIs to activate Intel Breadcrumbs
		//
		RHICORE_API void InitializeBeforeDeviceCreation(uint32 DeviceId);

		//
		// Called by platform RHIs when a GPU crash is detected.
		// Waits for for the processing to finish.
		//
		RHICORE_API bool OnGPUCrash();

		static inline constexpr TCHAR RootNodeName[] = TEXT("<root>");

		//
		// Platform RHI helper for implementing RHIBeginBreadcrumbGPU / RHIEndBreadcrumbGPU
		//
	#if WITH_RHI_BREADCRUMBS
		class FMarker
		{
		#if !INTEL_BREADCRUMBS_USE_BREADCRUMB_PTRS
			FRHIBreadcrumb::FBuffer Buffer;
		#endif

			void const* Ptr = nullptr;
			uint32 Size = 0;

		public:
			FMarker(FRHIBreadcrumbNode* Breadcrumb)
			{
			#if INTEL_BREADCRUMBS_USE_BREADCRUMB_PTRS
				// Intel Breadcrumbs will store a pointer to a breadcrumb node.
				Ptr = Breadcrumb ? Breadcrumb : FRHIBreadcrumbNode::Sentinel;
				Size = sizeof( FRHIBreadcrumbNode* );			// pointer size
			#else
				// Generate the breadcrumb node name
				TCHAR const* NameStr = Breadcrumb ? Breadcrumb->GetTCHAR(Buffer) : RootNodeName;
				Ptr = NameStr;
				Size = (FCString::Strlen(NameStr) + 1) * sizeof(TCHAR);// Include null terminator
			#endif
			}

			operator bool() const { return Ptr != nullptr; }

			void*  GetPtr () const { return const_cast<void*>(Ptr); }
			uint32 GetSize() const { return Size; }
		};
	#endif // WITH_RHI_BREADCRUMBS
	};

#endif // INTEL_GPU_CRASH_DUMPS
