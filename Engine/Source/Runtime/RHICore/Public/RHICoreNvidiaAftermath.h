// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef NV_AFTERMATH
#define NV_AFTERMATH 0
#endif

#if NV_AFTERMATH

	#include "RHIBreadcrumbs.h"

	namespace UE::RHICore::Nvidia::Aftermath
	{
		typedef TUniqueFunction<void(float TimeLimitSeconds, uint32 FrameLimit)> FAssociateShadersFunc;
		
		RHICORE_API bool IsEnabled();
		RHICORE_API bool IsShaderRegistrationEnabled();
		RHICORE_API bool IsResourceTrackingEnabled();
		RHICORE_API bool AreMarkersEnabled();

		//
		// Called by platform RHIs to activate Aftermath.
		//
		RHICORE_API void InitializeBeforeDeviceCreation();

		//
		// Called by the crash handler to associate shaders after a GPU fault
		//
		RHICORE_API void SetLateShaderAssociateCallback(FAssociateShadersFunc ShaderAssociateFunc = {});

		//
		// Called by platform RHIs after device creation.
		// InitCallback should return an Aftermath result value.
		//
		RHICORE_API bool InitializeDevice(TFunctionRef<uint32(uint32 AftermathFeatureFlags)> InitCallback);

		using FShaderHash = uint64;
		
		static constexpr FShaderHash InvalidShaderHash = UINT64_MAX;

		//
		// Register a new shader binary with Aftermath, must be valid.
		// Returns the internal shader hash, must be deregistered with DeregisterShaderBinary
		//
		RHICORE_API FShaderHash RegisterShaderBinary(const void* ShaderBinary, uint32 ByteSize, const FStringView& DebugName = nullptr);

		//
		// Deregister a shader binary from its hash. Will not release it immediately,
		// but instead enqueue it for later eviction once the GPU work is done.
		//
		RHICORE_API void DeregisterShaderBinary(FShaderHash Hash);

		struct FCrashResult
		{
			FString OutputLog;
			TOptional<FString> DumpPath;
			TOptional<uint64> GPUFaultAddress;
		};

		//
		// Called by platform RHIs when a GPU crash is detected.
		// Waits for Aftermath to finish crash dump processing, then returns the result.
		//
		RHICORE_API bool OnGPUCrash(TArray<FCrashResult>& OutResults);

		static inline constexpr TCHAR RootNodeName[] = TEXT("<root>");

		//
		// Platform RHI helper for implementing RHIBeginBreadcrumbGPU / RHIEndBreadcrumbGPU
		//
	#if WITH_RHI_BREADCRUMBS
		class FMarker
		{
			void const* Ptr = nullptr;
			uint32 Size = 0;

		public:
			FMarker(FRHIBreadcrumbNode* Breadcrumb)
			{
				if (!AreMarkersEnabled())
					return;

				// Have Aftermath store the breadcrumb node pointer directly.
				// Aftermath marker API fails if passed nullptr, so replace the tree root with the sentinel node.
				Ptr = Breadcrumb ? Breadcrumb : FRHIBreadcrumbNode::Sentinel;
			}

			operator bool() const { return Ptr != nullptr; }

			void*  GetPtr () const { return const_cast<void*>(Ptr); }
			uint32 GetSize() const { return Size; }
		};
	#endif // WITH_RHI_BREADCRUMBS
	};

#endif // NV_AFTERMATH
