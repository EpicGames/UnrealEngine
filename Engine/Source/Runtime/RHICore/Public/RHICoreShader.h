// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/App.h"
#include "RHICore.h"
#include "ShaderCore.h"
#include "ShaderParameterMacros.h"

struct FRHIShaderBundleComputeDispatch;
struct FRHIShaderBundleGraphicsDispatch;

namespace UE
{
namespace RHICore
{

/** Validates that the uniform buffer at the requested static slot. */
extern RHICORE_API void ValidateStaticUniformBuffer(FRHIUniformBuffer* UniformBuffer, FUniformBufferStaticSlot Slot, uint32 ExpectedHash);
extern RHICORE_API void SetupShaderCodeValidationData(FRHIShader* RHIShader, class FShaderCodeReader& ShaderCodeReader);
extern RHICORE_API void SetupShaderDiagnosticData(FRHIShader* RHIShader, class FShaderCodeReader& ShaderCodeReader);
extern RHICORE_API void RegisterDiagnosticMessages(const TArray<FShaderDiagnosticData>& In);
extern RHICORE_API const FString* GetDiagnosticMessage(uint32 MessageID);

/** Common implementations of dispatch shader bundle emulation shared by RHIs */

extern RHICORE_API void DispatchShaderBundleEmulation(
	FRHIComputeCommandList& InRHICmdList,
	FRHIShaderBundle* ShaderBundle,
	FRHIBuffer* ArgumentBuffer,
	TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
	TConstArrayView<FRHIShaderBundleComputeDispatch> Dispatches
);

extern RHICORE_API void DispatchShaderBundleEmulation(
	FRHICommandList& InRHICmdList,
	FRHIShaderBundle* ShaderBundle,
	FRHIBuffer* ArgumentBuffer,
	const FRHIShaderBundleGraphicsState& BundleState,
	TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
	TConstArrayView<FRHIShaderBundleGraphicsDispatch> Dispatches
);

inline void InitStaticUniformBufferSlots(FRHIShaderData* ShaderData)
{
	TArray<FUniformBufferStaticSlot>& StaticSlots = ShaderData->StaticSlots;
	const FShaderResourceTable& ShaderResourceTable = ShaderData->GetShaderResourceTable();

	StaticSlots.Reserve(ShaderResourceTable.ResourceTableLayoutHashes.Num());

	for (uint32 LayoutHash : ShaderResourceTable.ResourceTableLayoutHashes)
	{
		if (const FShaderParametersMetadata* Metadata = FindUniformBufferStructByLayoutHash(LayoutHash))
		{
			StaticSlots.Add(Metadata->GetLayout().StaticSlot);
		}
		else
		{
			StaticSlots.Add(MAX_UNIFORM_BUFFER_STATIC_SLOTS);
		}
	}
}

template <typename TApplyFunction>
void ApplyStaticUniformBuffers(
	const FRHIShader* Shader,
	const TArray<FRHIUniformBuffer*>& UniformBuffers,
	TApplyFunction&& ApplyFunction
)
{
	const TArray<uint32>& LayoutHashes = Shader->GetShaderResourceTable().ResourceTableLayoutHashes;
	const TArray<FUniformBufferStaticSlot>& Slots = Shader->GetStaticSlots();

	checkf(LayoutHashes.Num() == Slots.Num(), TEXT("Shader %s, LayoutHashes %d, Slots %d"),
		Shader->GetShaderName(), LayoutHashes.Num(), Slots.Num());

	for (int32 BufferIndex = 0; BufferIndex < Slots.Num(); ++BufferIndex)
	{
		const FUniformBufferStaticSlot Slot = Slots[BufferIndex];

		if (IsUniformBufferStaticSlotValid(Slot))
		{
			FRHIUniformBuffer* Buffer = UniformBuffers[Slot];
			ValidateStaticUniformBuffer(Buffer, Slot, LayoutHashes[BufferIndex]);

			if (Buffer)
			{
				ApplyFunction(BufferIndex, Buffer);
			}
		}
	}
}

template <typename TRHIContext, typename TRHIShader>
void ApplyStaticUniformBuffers(
	TRHIContext* CommandContext,
	const TRHIShader* Shader,
	const TArray<FRHIUniformBuffer*>& UniformBuffers)
{
	ApplyStaticUniformBuffers(Shader, UniformBuffers,
		[CommandContext, Shader](int32 BufferIndex, FRHIUniformBuffer* Buffer)
		{
			CommandContext->RHISetShaderUniformBuffer(Shader, BufferIndex, Buffer);
		});
}


} //! RHICore
} //! UE
