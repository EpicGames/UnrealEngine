// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RHITransientResourceAllocator.h"
#include "Algo/Partition.h"

#define RHICORE_TRANSIENT_ALLOCATOR_DEBUG (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)

#if RHICORE_TRANSIENT_ALLOCATOR_DEBUG
	#define IF_RHICORE_TRANSIENT_ALLOCATOR_DEBUG(Op) Op
#else
	#define IF_RHICORE_TRANSIENT_ALLOCATOR_DEBUG(Op)
#endif

inline uint64 ComputeHash(const FRHITextureCreateInfo& InCreateInfo, uint64 HeapOffset)
{
	// Make sure all padding is removed.
	FRHITextureCreateInfo NewInfo;
	FPlatformMemory::Memzero(&NewInfo, sizeof(FRHITextureCreateInfo));
	NewInfo = InCreateInfo;
	return CityHash64WithSeed((const char*)&NewInfo, sizeof(FRHITextureCreateInfo), HeapOffset);
}

inline uint64 ComputeHash(const FRHITextureCreateInfo& InCreateInfo)
{
	// Make sure all padding is removed.
	FRHITextureCreateInfo NewInfo;
	FPlatformMemory::Memzero(&NewInfo, sizeof(FRHITextureCreateInfo));
	NewInfo = InCreateInfo;
	return CityHash64((const char*)&NewInfo, sizeof(FRHITextureCreateInfo));
}

inline uint64 ComputeHash(const FRHIBufferCreateInfo& InCreateInfo, uint64 HeapOffset)
{
	// Make sure all padding is removed.
	FRHIBufferCreateInfo NewInfo;
	FPlatformMemory::Memzero(&NewInfo, sizeof(NewInfo));
	NewInfo = InCreateInfo;
	return CityHash64WithSeed(reinterpret_cast<const char*>(&NewInfo), sizeof(NewInfo), HeapOffset);
}

inline uint64 ComputeHash(const FRHIBufferCreateInfo& InCreateInfo)
{
	// Make sure all padding is removed.
	FRHIBufferCreateInfo NewInfo;
	FPlatformMemory::Memzero(&NewInfo, sizeof(NewInfo));
	NewInfo = InCreateInfo;
	return CityHash64(reinterpret_cast<const char*>(&NewInfo), sizeof(NewInfo));
}

/** Tracks allocation statistics for buffer or texture resources. */
struct FRHITransientResourceStats
{
	FRHITransientResourceStats() = default;

	void Add(const FRHITransientResourceStats& Other)
	{
		AllocatedSize     += Other.AllocatedSize;
		AllocationCount   += Other.AllocationCount;
		DeallocationCount += Other.DeallocationCount;
		CreateCount       += Other.CreateCount;
	}

	void Allocate(uint64 Size)
	{
		AllocationCount++;
		AllocatedSize      += Size;
	}

	void Deallocate(uint64 Size)
	{
		DeallocationCount++;
	}

	// The number of bytes allocated from the transient allocator this cycle.
	uint64 AllocatedSize = 0;

	// The number of allocations made from the transient allocator this cycle.
	uint32 AllocationCount = 0;

	// the number of deallocations made from the transient allocator this cycle.
	uint32 DeallocationCount = 0;

	// The number of resource creations made from the transient allocator this cycle.
	uint32 CreateCount = 0;
};

/** Tracks all transient memory statistics for the current allocation cycle and reports results to various profilers. */
struct FRHITransientMemoryStats
{
public:
	FRHITransientMemoryStats() = default;

	void Accumulate(const FRHITransientMemoryStats& Other)
	{
		Textures.Add(Other.Textures);
		Buffers.Add(Other.Buffers);
		AliasedSize = FMath::Max(AliasedSize, Other.AliasedSize);
	}

	void AllocateTexture(uint64 Size)
	{
		Textures.Allocate(Size);
		AliasedSizeCurrent += Size;
		AliasedSize         = FMath::Max(AliasedSize, AliasedSizeCurrent);
	}

	void DeallocateTexture(uint64 Size)
	{
		Textures.Deallocate(Size);
		AliasedSizeCurrent -= Size;
	}

	void AllocateBuffer(uint64 Size)
	{
		Buffers.Allocate(Size);
		AliasedSizeCurrent += Size;
		AliasedSize         = FMath::Max(AliasedSize, AliasedSizeCurrent);
	}

	void DeallocateBuffer(uint64 Size)
	{
		Buffers.Deallocate(Size);
		AliasedSizeCurrent -= Size;
	}

	void Reset()
	{
		Textures = {};
		Buffers = {};
		AliasedSize = AliasedSizeCurrent;
	}

	RHICORE_API void Submit(uint64 TotalMemoryCapacity);

	bool HasDeallocations() const { return Textures.DeallocationCount > 0 || Buffers.DeallocationCount > 0; }

	FRHITransientResourceStats Textures;
	FRHITransientResourceStats Buffers;

	// Total allocated memory usage with aliasing.
	uint64 AliasedSizeCurrent = 0;

	// Current aliased size as items are being allocated / deallocated.
	uint64 AliasedSize = 0;
};

/** An RHI transient resource cache designed to optimize fetches for resources placed into a heap with an offset.
 *  The cache has a fixed capacity whereby no garbage collection will occur. Once that capacity is exceeded, garbage
 *  collection is invoked on resources older than a specified generation (where generation is incremented with each
 *  cycle of forfeiting acquired resources).
 */
template <typename TransientResourceType>
class TRHITransientResourceCache
{
public:
	static const uint32 kInfinity = ~0u;
	static const uint32 kDefaultCapacity = kInfinity;
	static const uint32 kDefaultGarbageCollectLatency = 32;

	TRHITransientResourceCache(
		uint32 InCapacity = kDefaultCapacity,
		uint32 InGarbageCollectLatency = kDefaultGarbageCollectLatency)
		: GarbageCollectLatency(InGarbageCollectLatency)
		, Capacity(InCapacity)
	{
		if (Capacity != kInfinity)
		{
			Cache.Reserve(Capacity);
		}
	}

	~TRHITransientResourceCache()
	{
		for (const FCacheItem& Item : Cache)
		{
			delete Item.Resource;
		}

		for (TransientResourceType* Resource : Allocated)
		{
			delete Resource;
		}
	}

	template <typename CreateFunctionType>
	TransientResourceType* Acquire(uint64 Hash, CreateFunctionType CreateFunction)
	{
		for (int32 Index = 0; Index < Cache.Num(); ++Index)
		{
			const FCacheItem& CacheItem = Cache[Index];

			if (CacheItem.Hash == Hash)
			{
				TransientResourceType* Resource = CacheItem.Resource;
				Cache.RemoveAtSwap(Index, EAllowShrinking::No);
				Allocated.Emplace(Resource);
				HitCount++;
				return Resource;
			}
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(CreatePlacedResource);
		TransientResourceType* Resource = CreateFunction(Hash);
		Allocated.Emplace(Resource);
		MissCount++;
		return Resource;
	}

	template <typename ReleaseFunctionType>
	void Forfeit(uint64 CurrentFrameIndex, ReleaseFunctionType ReleaseFunction)
	{
		const int32 FirstForfeitIndex = Algo::Partition(Allocated.GetData(), Allocated.Num(), [](const FRHITransientResource* Resource) { return Resource->IsAcquired(); });
		const auto ResourcesToForfeit = MakeArrayView(Allocated.GetData() + FirstForfeitIndex, Allocated.Num() - FirstForfeitIndex);

		for (TransientResourceType* Resource : ResourcesToForfeit)
		{
			Cache.Emplace(Resource, Resource->GetHash(), CurrentFrameIndex);
		}

		Allocated.SetNum(FirstForfeitIndex, EAllowShrinking::No);

		Algo::Sort(Cache, [](const FCacheItem& LHS, const FCacheItem& RHS)
		{
			return LHS.LastUsedFrame > RHS.LastUsedFrame;
		});

		while (uint32(Cache.Num()) > Capacity)
		{
			if (!TryReleaseItem(CurrentFrameIndex, ReleaseFunction))
			{
				break;
			}
		}

		HitCount = 0;
		MissCount = 0;
	}

	void Forfeit(uint64 CurrentFrameIndex)
	{
		Forfeit(CurrentFrameIndex, [](TransientResourceType*) {});
	}

	TConstArrayView<TransientResourceType*> GetAllocated() const { return Allocated; }

	uint32 GetAllocatedCount() const { return Allocated.Num(); }

	uint32 GetSize() const { return Cache.Num(); }

	uint32 GetCapacity() const { return Capacity; }

	uint32 GetHitCount() const { return HitCount; }

	uint32 GetMissCount() const { return MissCount; }

	float GetHitPercentage() const { return (float)HitCount / (float)(HitCount + MissCount); }

private:
	template <typename ReleaseFunctionType>
	bool TryReleaseItem(uint64 CurrentFrameIndex, ReleaseFunctionType ReleaseFunction)
	{
		const FCacheItem& Item = Cache.Top();

		if (Item.LastUsedFrame + GarbageCollectLatency <= CurrentFrameIndex)
		{
			ReleaseFunction(Item.Resource);
			delete Item.Resource;
			Cache.Pop();
			return true;
		}

		return false;
	}

	struct FCacheItem
	{
		FCacheItem(TransientResourceType* InResource, uint64 InHash, uint64 InLastUsedFrame)
			: Resource(InResource)
			, Hash(InHash)
			, LastUsedFrame(InLastUsedFrame)
		{}

		TransientResourceType* Resource;
		uint64 Hash{};
		uint64 LastUsedFrame{};
	};

	TArray<FCacheItem> Cache;
	TArray<TransientResourceType*> Allocated;
	uint32 GarbageCollectLatency;
	uint32 Capacity;
	uint32 HitCount = 0;
	uint32 MissCount = 0;
};

///////////////////////////////////////////////////////////////////////////////////////////////////

/** Abstract base class for memory caching, providing a common garbage collection method. */
class IRHITransientMemoryCache
{
public:
	virtual ~IRHITransientMemoryCache() = default;

	virtual void GarbageCollect() = 0;
};

///////////////////////////////////////////////////////////////////////////////////////////////////
//! Transient Resource Heap Allocator
///////////////////////////////////////////////////////////////////////////////////////////////////

class FRHITransientHeapCache;
class FRHITransientResourceHeapAllocator;

/** First-fit allocator used for placing resources on a heap. */
class FRHITransientHeapAllocator
{
public:
	struct FAliasingOverlap
	{
		FAliasingOverlap() = default;
		FAliasingOverlap(FRHITransientResource* InResource, uint32 InAcquireFence)
			: Resource(InResource)
			, AcquireFence(InAcquireFence)
		{}

		FRHITransientResource* Resource;
		uint32 AcquireFence;
	};

	RHICORE_API FRHITransientHeapAllocator(uint64 Capacity, uint32 Alignment);

	RHICORE_API FRHITransientHeapAllocation Allocate(const FRHITransientAllocationFences& Fences, uint64 Size, uint32 Alignment, TArray<FAliasingOverlap>& OutAliasingOverlaps);

	RHICORE_API void Deallocate(FRHITransientResource* Resource, const FRHITransientAllocationFences& Fences);

	RHICORE_API void Flush();

	void SetGpuVirtualAddress(uint64 InGpuVirtualAddress)
	{
		GpuVirtualAddress = InGpuVirtualAddress;
	}

	uint64 GetGpuVirtualAddress() const { return GpuVirtualAddress; }
	uint64 GetCapacity() const { return Capacity; }
	uint64 GetUsedSize() const { return UsedSize; }
	uint64 GetFreeSize() const { return Capacity - UsedSize; }
	uint64 GetAlignmentWaste() const { return AlignmentWaste; }
	uint32 GetAllocationCount() const { return AllocationCount; }

	bool IsFull() const { return UsedSize == Capacity; }
	bool IsEmpty() const { return UsedSize == 0; }

private:
	using FRangeHandle = uint16;
	static const FRangeHandle InvalidRangeHandle = FRangeHandle(~0);

	struct FRange
	{
		FRHITransientResource* Resource = nullptr;
		FRHITransientAllocationFences Fences;
		uint64 Size = 0;
		uint64 Offset = 0;
		FRangeHandle NextFreeHandle = InvalidRangeHandle;
		FRangeHandle PrevFreeHandle = InvalidRangeHandle;

		inline uint64 GetStart() const { return Offset; }
		inline uint64 GetEnd() const { return Size + Offset; }
	};

	inline FRangeHandle GetFirstFreeRangeHandle()
	{
		return Ranges[HeadHandle].NextFreeHandle;
	}

	FRangeHandle CreateRange()
	{
		if (!RangeFreeList.IsEmpty())
		{
			return RangeFreeList.Pop();
		}
		Ranges.Emplace();
		return FRangeHandle(Ranges.Num() - 1);
	}

	FRangeHandle InsertRange(FRangeHandle PreviousHandle, FRHITransientResource* Resource, const FRHITransientAllocationFences& Fences, uint64 Offset, uint64 Size)
	{
		FRangeHandle Handle = CreateRange();

		FRange& CurrentRange = Ranges[Handle];
		CurrentRange.Resource = Resource;
		CurrentRange.Fences = Fences;
		CurrentRange.Offset = Offset;
		CurrentRange.Size = Size;

		FRange& PreviousRange = Ranges[PreviousHandle];
		CurrentRange.NextFreeHandle = PreviousRange.NextFreeHandle;
		PreviousRange.NextFreeHandle = Handle;

		return Handle;
	}

	FRangeHandle RemoveRange(FRangeHandle PreviousHandle, FRangeHandle CurrentHandle)
	{
		FRange& PreviousRange = Ranges[PreviousHandle];
		FRange& CurrentRange = Ranges[CurrentHandle];

		FRangeHandle NextCurrentHandle = CurrentRange.NextFreeHandle;

		check(PreviousRange.NextFreeHandle == CurrentHandle);
		PreviousRange.NextFreeHandle = CurrentRange.NextFreeHandle;
		CurrentRange.NextFreeHandle = InvalidRangeHandle;
		CurrentRange.Resource = nullptr;

		RangeFreeList.Add(CurrentHandle);
		return NextCurrentHandle;
	}

	struct FFindResult
	{
		uint64 LeftoverSize = 0;
		FRangeHandle PreviousHandle = InvalidRangeHandle;
		FRangeHandle FoundHandle = InvalidRangeHandle;
	};

	RHICORE_API void Validate();

	uint64 GpuVirtualAddress = 0;
	uint64 Capacity = 0;
	uint64 UsedSize = 0;
	uint64 AlignmentWaste = 0;
	uint32 AllocationCount = 0;
	uint32 AlignmentMin = 0;

	FRangeHandle HeadHandle = InvalidRangeHandle;
	TArray<FRangeHandle> RangeFreeList;
	TArray<FRange> Ranges;
};

///////////////////////////////////////////////////////////////////////////////////////////////////

enum class ERHITransientHeapFlags : uint8
{
	// Supports placing buffers onto the heap.
	AllowBuffers = 1 << 0,

	// Supports placing textures with UAV support onto the heap.
	AllowTextures = 1 << 1,

	// Supports placing render targets onto the heap.
	AllowRenderTargets = 1 << 2,

	// Supports placing NNE accessible buffers onto the heap.  Differentiation is required for DirectML with multi-GPU.
	AllowNNEBuffers = 1 << 3,

	// Supports all resource types.
	AllowAll = AllowBuffers | AllowTextures | AllowRenderTargets | AllowNNEBuffers
};

ENUM_CLASS_FLAGS(ERHITransientHeapFlags);

/** The base class for a platform heap implementation. Transient resources are placed on the heap at specific
 *  byte offsets. Each heap additionally contains a cache of RHI transient resources, each with its own RHI
 *  resource and cache of RHI views. The lifetime of the resource cache is tied to the heap.
 */
class FRHITransientHeap
{
public:
	struct FInitializer
	{
		// Size of the heap in bytes.
		uint64 Size = 0;

		// Alignment of the heap in bytes.
		uint32 Alignment = 0;

		// Flags used to filter resource allocations within the heap.
		ERHITransientHeapFlags Flags = ERHITransientHeapFlags::AllowAll;

		// Size of the texture cache before elements are evicted.
		uint32 TextureCacheSize = 0;

		// Size of the buffer cache before elements are evicted.
		uint32 BufferCacheSize = 0;
	};

	struct FResourceInitializer
	{
		FResourceInitializer(const FRHITransientHeapAllocation& InAllocation, uint64 InHash)
			: Heap(*InAllocation.Heap)
			, Allocation(InAllocation)
			, Hash(InHash)
		{}

		// The heap on which to create the resource.
		FRHITransientHeap& Heap;

		// The allocation (offset / size) on the provided heap.
		const FRHITransientHeapAllocation& Allocation;

		// The unique hash computed from the create info and allocation offset.
		const uint64 Hash;
	};

	using FCreateTextureFunction = TFunction<FRHITransientTexture* (const FResourceInitializer&)>;
	using FCreateBufferFunction = TFunction<FRHITransientBuffer* (const FResourceInitializer&)>;

	FRHITransientHeap(const FInitializer& InInitializer)
		: Initializer(InInitializer)
		, Allocator(InInitializer.Size, InInitializer.Alignment)
		, AlignmentLog2(FPlatformMath::CeilLogTwo64(InInitializer.Alignment))
		, Textures(InInitializer.TextureCacheSize)
		, Buffers(InInitializer.BufferCacheSize)
	{
		check(1ull << AlignmentLog2 == InInitializer.Alignment);
	}

	virtual ~FRHITransientHeap() = default;

	RHICORE_API FRHITransientTexture* CreateTexture(
		const FRHITextureCreateInfo& CreateInfo,
		const TCHAR* DebugName,
		const FRHITransientAllocationFences& Fences,
		uint64 CurrentAllocatorCycle,
		uint64 TextureSize,
		uint32 TextureAlignment,
		FCreateTextureFunction CreateTextureFunction);

	RHICORE_API void DeallocateMemory(FRHITransientTexture* Texture, const FRHITransientAllocationFences& Fences);

	RHICORE_API FRHITransientBuffer* CreateBuffer(
		const FRHIBufferCreateInfo& CreateInfo,
		const TCHAR* DebugName,
		const FRHITransientAllocationFences& Fences,
		uint64 CurrentAllocatorCycle,
		uint64 BufferSize,
		uint32 BufferAlignment,
		FCreateBufferFunction CreateBufferFunction);

	RHICORE_API void DeallocateMemory(FRHITransientBuffer* Buffer, const FRHITransientAllocationFences& Fences);

	RHICORE_API void Flush(uint64 CurrentAllocatorCycle, FRHITransientMemoryStats& OutMemoryStats, FRHITransientAllocationStats* OutAllocationStats);

	const FInitializer& GetInitializer() const { return Initializer; }

	uint64 GetCapacity() const { return Allocator.GetCapacity(); }

	uint64 GetGPUVirtualAddress() const { return Allocator.GetGpuVirtualAddress(); }

	uint64 GetLastUsedGarbageCollectCycle() const { return LastUsedGarbageCollectCycle; }

	uint64 GetCommitSize() const { return CommitSize; }

	bool IsEmpty() const { return Allocator.IsEmpty(); }

	bool IsFull() const { return Allocator.IsFull(); }

	bool IsCommitRequired() const { return CommitSize > 0; }

	bool IsAllocationSupported(uint64 Size, ERHITransientHeapFlags Flags) const
	{
		return Size <= Allocator.GetFreeSize() && EnumHasAnyFlags(Initializer.Flags, Flags);
	}

protected:
	void SetGpuVirtualAddress(uint64 InBaseGPUVirtualAddress)
	{
		Allocator.SetGpuVirtualAddress(InBaseGPUVirtualAddress);
	}

private:
	RHICORE_API void AllocateMemoryInternal(FRHITransientResource* Resource, const FRHITransientHeapAllocation& Allocation);
	RHICORE_API void DeallocateMemoryInternal(FRHITransientResource* Resource, const FRHITransientAllocationFences& Fences);

	FInitializer Initializer;
	FRHITransientHeapAllocator Allocator;

	uint64 LastUsedGarbageCollectCycle = 0;
	uint64 CommitSize = 0;
	std::atomic<uint64> CommitSizeMax{0};
	uint32 AlignmentLog2;

	TArray<FRHITransientHeapAllocator::FAliasingOverlap> AliasingOverlaps;

	FRHITransientMemoryStats Stats;
	TRHITransientResourceCache<FRHITransientTexture> Textures;
	TRHITransientResourceCache<FRHITransientBuffer> Buffers;

	friend FRHITransientHeapCache;
	friend FRHITransientResourceHeapAllocator;
};

///////////////////////////////////////////////////////////////////////////////////////////////////

/** The RHI transient heap system is a base class for the platform implementation. It has a persistent lifetime
 *  and contains a cache of transient heaps. The transient allocator acquires heaps from the system and forfeits them
 *  at the end of its lifetime. Garbage collection of heaps is done using an internal counter that increments with
 *  each GarbageCollect call. This should be done periodically. Heaps older than a platform-specified fence latency
 *  are destroyed. Additionally, statistics are gathered automatically and reported to the 'rhitransientmemory' stats
 *  group.
 */
class FRHITransientHeapCache : public IRHITransientMemoryCache
{
public:
	struct FInitializer
	{
		// Creates a default initializer using common RHI CVars.
		static RHICORE_API FInitializer CreateDefault();

		static const uint32 kDefaultResourceCacheSize = 256;

		// The minimum size to use when creating the first heap. This is the default but can grow based on allocations.
		uint64 MinimumFirstHeapSize = 0;

		// The minimum size to use when creating a heap. This is the default but can grow based on allocations.
		uint64 MinimumHeapSize = 0;

		// The minimum alignment for resources in the heap.
		uint32 HeapAlignment = 0;

		// The latency between the completed fence value and the used fence value to invoke garbage collection of the heap.
		uint32 GarbageCollectLatency = 0;

		// Size of the texture cache before elements are evicted.
		uint32 TextureCacheSize = kDefaultResourceCacheSize;

		// Size of the buffer cache before elements are evicted.
		uint32 BufferCacheSize = kDefaultResourceCacheSize;

		// Whether all heaps should be created with the AllowAll heap flag.
		bool bSupportsAllHeapFlags = true;

		// Whether all heaps support mapping physical pages to the commit size. If false the physical memory usage is represented by the capacity instead.
		bool bSupportsVirtualMapping = false;
	};

	FRHITransientHeapCache(const FInitializer& InInitializer)
		: Initializer(InInitializer)
	{}

	RHICORE_API virtual ~FRHITransientHeapCache();

	RHICORE_API FRHITransientHeap* Acquire(uint64 FirstAllocationSize, ERHITransientHeapFlags FirstAllocationHeapFlags);

	RHICORE_API void Forfeit(TConstArrayView<FRHITransientHeap*> Heaps);

	RHICORE_API void GarbageCollect() override;

	const FInitializer& GetInitializer() const { return Initializer; }

	uint64 GetGarbageCollectCycle() const { return GarbageCollectCycle; }

	uint64 GetHeapSize(uint64 RequestedHeapSize) const
	{
		return FMath::Max(FMath::RoundUpToPowerOfTwo64(RequestedHeapSize), Initializer.MinimumHeapSize);
	}

private:
	//////////////////////////////////////////////////////////////////////////
	//! Platform API

	// Called when a new heap is being created and added to the pool.
	virtual FRHITransientHeap* CreateHeap(const FRHITransientHeap::FInitializer& Initializer) = 0;

	//////////////////////////////////////////////////////////////////////////

	FInitializer Initializer;
	FRHITransientMemoryStats Stats;

	FCriticalSection CriticalSection;
	TArray<FRHITransientHeap*> LiveList;
	TArray<FRHITransientHeap*> FreeList;
	uint64 GarbageCollectCycle = 0;

	friend FRHITransientResourceHeapAllocator;
};

/** A base class for implementing IRHITransientResourceAllocator for a virtual aliasing placed resource heap allocation strategy. */
class FRHITransientResourceHeapAllocator : public IRHITransientResourceAllocator
{
public:
	RHICORE_API FRHITransientResourceHeapAllocator(FRHITransientHeapCache& InHeapCache);
	RHICORE_API ~FRHITransientResourceHeapAllocator();

	// Sets the create mode for allocations.
	RHICORE_API void SetCreateMode(ERHITransientResourceCreateMode InCreateMode) override;

	// Deallocates a texture from its parent heap. Provide the current platform fence value used to update the heap.
	RHICORE_API void DeallocateMemory(FRHITransientTexture* Texture, const FRHITransientAllocationFences& Fences) override;

	// Deallocates a buffer from its parent heap. Provide the current platform fence value used to update the heap.
	RHICORE_API void DeallocateMemory(FRHITransientBuffer* Buffer, const FRHITransientAllocationFences& Fences) override;

	// Called to flush any active allocations prior to rendering.
	RHICORE_API void Flush(FRHICommandListImmediate& RHICmdList, FRHITransientAllocationStats* OutAllocationStats) override;

	// Returns the array of heaps used by this allocator, including the required commit size for each.
	inline TConstArrayView<FRHITransientHeap*> GetHeaps() const { return Heaps; }

	FRHITransientHeapCache& HeapCache;

	template <typename TransientResourceType, typename LambdaType, typename ResourceCreateInfo>
	TransientResourceType* CreateTransientResource(LambdaType&& Lambda, uint64 Hash, uint64 Size, const ResourceCreateInfo& CreateInfo)
	{
		TransientResourceType* Resource;
		if (CreateMode == ERHITransientResourceCreateMode::Inline)
		{
			typename TransientResourceType::FResourceTaskResult TaskResult = Lambda();
			Resource = new TransientResourceType(TaskResult.Resource.GetReference(), TaskResult.GpuVirtualAddress, Hash, Size, ERHITransientAllocationType::Heap, CreateInfo);
		}
		else
		{
			Resource = new TransientResourceType(UE::Tasks::Launch(UE_SOURCE_LOCATION, MoveTemp(Lambda), LowLevelTasks::ETaskPriority::High), Hash, Size, ERHITransientAllocationType::Heap, CreateInfo);
		}
		return Resource;
	}

protected:
	/** Allocates a texture on a heap at a specific offset, returning a cached RHI transient texture pointer, or null
	 *  if the allocation failed. TextureSize and TextureAlignment are platform specific and must be derived from the
	 *  texture create info and passed in, along with a platform-specific texture creation function if no cached resource
	 *  if found.
	 */
	RHICORE_API FRHITransientTexture* CreateTextureInternal(
		const FRHITextureCreateInfo& CreateInfo,
		const TCHAR* DebugName,
		const FRHITransientAllocationFences& Fences,
		uint64 TextureSize,
		uint32 TextureAlignment,
		FRHITransientHeap::FCreateTextureFunction CreateTextureFunction);

	/** Allocates a buffer on a heap at a specific offset, returning a cached RHI transient buffer pointer, or null
	 *  if the allocation failed. BufferSize and BufferAlignment are platform specific and must be derived from the
	 *  buffer create info and passed in, along with a platform-specific buffer creation function if no cached resource
	 *  if found.
	 */
	RHICORE_API FRHITransientBuffer* CreateBufferInternal(
		const FRHIBufferCreateInfo& CreateInfo,
		const TCHAR* DebugName,
		const FRHITransientAllocationFences& Fences,
		uint32 BufferSize,
		uint32 BufferAlignment,
		FRHITransientHeap::FCreateBufferFunction CreateBufferFunction);

private:
	TArray<FRHITransientHeap*> Heaps;
	uint64 CurrentCycle = 0;
	uint32 DeallocationCount = 0;
	ERHITransientResourceCreateMode CreateMode = ERHITransientResourceCreateMode::Inline;

	IF_RHICORE_TRANSIENT_ALLOCATOR_DEBUG(TSet<FRHITransientResource*> ActiveResources);
};

///////////////////////////////////////////////////////////////////////////////////////////////////
//! Transient Resource Page Allocator
///////////////////////////////////////////////////////////////////////////////////////////////////

class FRHITransientPagePoolCache;
class FRHITransientResourcePageAllocator;

/** Allocates page spans for a resource. */
class FRHITransientPageSpanAllocator
{
public:
	FRHITransientPageSpanAllocator(uint32 InPageCount, uint32 InPageSize)
		: MaxSpanCount(InPageCount + 2)
		, MaxPageCount(InPageCount)
		, PageSize(InPageSize)
	{
		Init();
	}

	RHICORE_API void Reset();

	RHICORE_API bool Allocate(FRHITransientResource* Resource, const FRHITransientAllocationFences& Fences, uint32 PageCount, uint32& NumPagesAllocated, uint32& OutSpanIndex);

	RHICORE_API void Deallocate(FRHITransientResource* Resource, const FRHITransientAllocationFences& Fences, uint32 SpanIndex);

	RHICORE_API void Flush();

	template <typename SpanArrayType>
	void GetSpanArray(uint32 SpanIndex, SpanArrayType& OutPageSpans) const
	{
		check(SpanIndex != InvalidIndex);
		do
		{
			OutPageSpans.Emplace(PageSpans[SpanIndex]);
			SpanIndex = PageSpans[SpanIndex].NextSpanIndex;
		} while (SpanIndex != InvalidIndex);
	}

	uint32 GetAllocationCount() const { return AllocationCount; }

	uint32 GetFreePageCount() const { return FreePageCount; }

	uint64 GetUsedSize() const { return (MaxPageCount - FreePageCount) * PageSize; }

	uint64 GetFreeSize() const { return FreePageCount * PageSize; }

	RHICORE_API uint32 GetAllocationPageCount(uint32 SpanIndex) const;

	uint32 GetMaxSpanCount() const { return MaxSpanCount; }

	uint32 GetPageSize() const { return PageSize; }

	uint64 GetCapacity() const { return MaxPageCount * PageSize; }

	bool IsFull() const { return FreePageCount == 0; }

	bool IsEmpty() const { return AllocationCount == 0; }

private:
	static const uint32 InvalidIndex = TNumericLimits<uint32>::Max();
	static const uint32 FreeSpanListHeadIndex = 0;
	static const uint32 FreeSpanListTailIndex = 1;

	struct FPageSpan : FRHITransientPageSpan
	{
		const bool IsLinked() { return (NextSpanIndex != InvalidIndex || PrevSpanIndex != InvalidIndex); }

		FRHITransientResource* Resource = nullptr;
		FRHITransientAllocationFences Fences;
		uint32 NextSpanIndex = InvalidIndex;
		uint32 PrevSpanIndex = 0;
		bool bAllocated = false;
	};

	RHICORE_API void Init();

	// Splits a span into two, so that the original span has PageCount pages and the new span contains the remaining ones
	RHICORE_API void SplitSpan(uint32 SpanIndex, uint32 PageCount);

	// Merges two spans. They must be adjacent and in the same list
	RHICORE_API void MergeSpans(uint32 SpanIndex0, uint32 SpanIndex1);

	// Inserts a span after an existing span. The span to insert must be unlinked
	RHICORE_API void InsertAfter(uint32 InsertPosition, uint32 InsertSpanIndex);

	// Inserts a span after an existing span. The span to insert must be unlinked
	RHICORE_API void InsertBefore(uint32 InsertPosition, uint32 InsertSpanIndex);

	// Removes a span from its list, reconnecting neighbouring list elements
	RHICORE_API void Unlink(uint32 SpanIndex);

	// Allocates an unused span from the pool
	int AllocSpan()
	{
		check(UnusedSpanListCount > 0);
		uint32 SpanIndex = UnusedSpanList[UnusedSpanListCount - 1];
		UnusedSpanListCount--;
		return SpanIndex;
	}

	// Releases a span back to the unused pool
	void ReleaseSpan(uint32 SpanIndex)
	{
		check(!PageSpans[SpanIndex].IsLinked());
		UnusedSpanList[UnusedSpanListCount] = SpanIndex;
		UnusedSpanListCount++;
		check(UnusedSpanListCount <= MaxPageCount);
	}

	RHICORE_API void Validate();

	uint32 GetFirstSpanIndex() const
	{
		return PageSpans[FreeSpanListHeadIndex].NextSpanIndex;
	}

	TArray<int32> PageToSpanStart;  // [PAGE_COUNT + 1]
	TArray<int32> PageToSpanEnd;    // [PAGE_COUNT + 1]
	TArray<FPageSpan> PageSpans;    // [MAX_SPAN_COUNT]
	TArray<int32> UnusedSpanList;   // [MAX_SPAN_COUNT]

	uint32 FreePageCount;
	uint32 UnusedSpanListCount;

	const uint32 MaxSpanCount;
	const uint32 MaxPageCount;
	const uint32 PageSize;
	uint32 AllocationCount;
};

///////////////////////////////////////////////////////////////////////////////////////////////////

class FRHITransientPagePool
{
public:
	struct FInitializer
	{
		uint32 PageCount = 0;
		uint32 PageSize = 0;
	};

	FRHITransientPagePool(const FInitializer& InInitializer)
		: Initializer(InInitializer)
		, Allocator(Initializer.PageCount, Initializer.PageSize)
	{}

	virtual ~FRHITransientPagePool() = default;

	struct FAllocationContext
	{
		FAllocationContext(FRHITransientResource& InResource, const FRHITransientAllocationFences& InFences, uint32 InPageSize)
			: Resource(InResource)
			, Allocations(InResource.GetPageAllocation().PoolAllocations)
			, Spans(InResource.GetPageAllocation().Spans)
			, AllocationsBefore(Allocations)
			, GpuVirtualAddress(Resource.GetGpuVirtualAddress())
			, Size(Align(Resource.GetSize(), InPageSize))
			, Fences(InFences)
			, PagesRemaining(Size / InPageSize)
		{
			Allocations.Reset();
			Spans.Reset();
		}

		bool IsComplete() const { return PagesRemaining == 0; }

		FRHITransientResource& Resource;
		TArray<FRHITransientPagePoolAllocation>& Allocations;
		TArray<FRHITransientPageSpan>& Spans;
		const TArray<FRHITransientPagePoolAllocation, TInlineAllocator<8>> AllocationsBefore;
		const uint64 GpuVirtualAddress;
		const uint64 Size;
		const FRHITransientAllocationFences Fences;

		uint32 AllocationCount = 0;
		uint32 AllocationMatchingCount = 0;
		uint32 PagesRemaining = 0;
		uint32 MaxAllocationPage = 0;
		uint32 PagesAllocated = 0;
		uint32 PageSpansAllocated = 0;
		uint32 PagesMapped = 0;
	};

	RHICORE_API void Allocate(FAllocationContext& AllocationContext);

	void Deallocate(FRHITransientResource* Resource, const FRHITransientAllocationFences& Fences, uint32 SpanIndex)
	{
		Allocator.Deallocate(Resource, Fences, SpanIndex);
	}

	RHICORE_API void Flush(FRHICommandListImmediate& RHICmdList);

	uint64 GetLastUsedGarbageCollectCycle() const { return LastUsedGarbageCollectCycle; }

	bool IsEmpty() const { return Allocator.IsEmpty(); }

	bool IsFull() const { return Allocator.IsFull(); }

	uint64 GetCapacity() const { return Allocator.GetCapacity(); }

	uint64 GetGpuVirtualAddress() const { return GpuVirtualAddress; }

	struct FPageMapRequest
	{
		FPageMapRequest() = default;

		FPageMapRequest(
			uint64 InDestinationAddress,
			uint64 InSourcePagePoolAddress,
			uint32 InSourcePageCount,
			uint32 InPageSpanOffset,
			uint32 InPageSpanCount)
			: DestinationAddress(InDestinationAddress)
			, SourcePagePoolAddress(InSourcePagePoolAddress)
			, SourcePageCount(InSourcePageCount)
			, PageSpanOffset(InPageSpanOffset)
			, PageSpanCount(InPageSpanCount)
		{}

		uint64 DestinationAddress = 0;
		uint64 SourcePagePoolAddress = 0;
		uint32 SourcePageCount = 0;
		uint32 PageSpanOffset = 0;
		uint32 PageSpanCount = 0;
	};

	const FInitializer Initializer;

protected:
	void SetGpuVirtualAddress(uint64 InGpuVirtualAddress)
	{
		check(InGpuVirtualAddress != 0);
		GpuVirtualAddress = InGpuVirtualAddress;
	}

private:
	//////////////////////////////////////////////////////////////////////////
	//! Platform API

	virtual void Flush(FRHICommandListImmediate& RHICmdList, TArray<FPageMapRequest>&& InPageMapRequests, TArray<FRHITransientPageSpan>&& InPageSpans) = 0;

	//////////////////////////////////////////////////////////////////////////

	FRHITransientPageSpanAllocator Allocator;

	TArray<FPageMapRequest> PageMapRequests;
	TArray<FRHITransientPageSpan> PageSpans;
	uint64 GpuVirtualAddress = 0;
	uint64 LastUsedGarbageCollectCycle = 0;
	uint32 PageMapRequestCountMax = 0;
	uint32 PageSpanCountMax = 0;

	friend FRHITransientPagePoolCache;
	friend FRHITransientResourcePageAllocator;
};

///////////////////////////////////////////////////////////////////////////////////////////////////

class FRHITransientPagePoolCache : public IRHITransientMemoryCache
{
public:
	struct FInitializer
	{
		// Creates a default initializer using common RHI CVars.
		static RHICORE_API FInitializer CreateDefault();

		static const uint32 kDefaultResourceCacheSize = 256;

		// Size in bytes of the pool. Must be a multiple of PageSize.
		uint64 PoolSize = 0;

		// Size in bytes of the first pool. Only takes effect if larger than PoolSize.
		uint64 PoolSizeFirst = 0;

		// Size of each page.
		uint32 PageSize = 0;

		// The latency between the completed fence value and the used fence value to invoke garbage collection of the heap.
		uint32 GarbageCollectLatency = 0;

		// Size of the texture cache before elements are evicted.
		uint32 TextureCacheSize = kDefaultResourceCacheSize;

		// Size of the buffer cache before elements are evicted.
		uint32 BufferCacheSize = kDefaultResourceCacheSize;
	};

	FRHITransientPagePoolCache(const FInitializer& InInitializer)
		: Initializer(InInitializer)
	{}

	FRHITransientPagePoolCache(const FRHITransientPagePoolCache&) = delete;

	RHICORE_API virtual ~FRHITransientPagePoolCache();

	// Called by the transient allocator to acquire a page pool from the cache.
	RHICORE_API FRHITransientPagePool* Acquire();

	// Called by the transient allocator to return the fast page pool if it exists.
	RHICORE_API FRHITransientPagePool* GetFastPagePool();

	// Called by the transient allocator to forfeit all acquired heaps back to the cache.
	RHICORE_API void Forfeit(TConstArrayView<FRHITransientPagePool*> PagePools);

	RHICORE_API void GarbageCollect() override;

	const FInitializer Initializer;

private:
	//////////////////////////////////////////////////////////////////////////
	//! Platform API

	// Called to access the dedicated fast VRAM page pool, if it exists on the platform.
	virtual FRHITransientPagePool* CreateFastPagePool() { return nullptr; }

	// Called when a new heap is being created and added to the pool.
	virtual FRHITransientPagePool* CreatePagePool(const FRHITransientPagePool::FInitializer& Initializer) = 0;

	//////////////////////////////////////////////////////////////////////////

	FRHITransientMemoryStats Stats;

	FCriticalSection CriticalSection;
	FRHITransientPagePool* FastPagePool = nullptr;
	TArray<FRHITransientPagePool*> LiveList;
	TArray<FRHITransientPagePool*> FreeList;
	uint64 GarbageCollectCycle = 0;
	uint64 TotalMemoryCapacity = 0;

	friend FRHITransientResourcePageAllocator;
};

///////////////////////////////////////////////////////////////////////////////////////////////////

class FRHITransientResourcePageAllocator : public IRHITransientResourceAllocator
{
public:
	FRHITransientResourcePageAllocator(FRHITransientPagePoolCache& InPagePoolCache)
		: PagePoolCache(InPagePoolCache)
		, Textures(InPagePoolCache.Initializer.TextureCacheSize, ResourceCacheGarbageCollectionLatency)
		, Buffers(InPagePoolCache.Initializer.BufferCacheSize, ResourceCacheGarbageCollectionLatency)
		, PageSize(PagePoolCache.Initializer.PageSize)
	{
		FastPagePool = PagePoolCache.GetFastPagePool();
	}

	RHICORE_API FRHITransientTexture* CreateTexture(const FRHITextureCreateInfo& CreateInfo, const TCHAR* DebugName, const FRHITransientAllocationFences& Fences) override;
	RHICORE_API FRHITransientBuffer* CreateBuffer(const FRHIBufferCreateInfo& CreateInfo, const TCHAR* DebugName, const FRHITransientAllocationFences& Fences) override;
	RHICORE_API void DeallocateMemory(FRHITransientTexture* Texture, const FRHITransientAllocationFences& Fences) override;
	RHICORE_API void DeallocateMemory(FRHITransientBuffer* Buffer, const FRHITransientAllocationFences& Fences) override;
	RHICORE_API void Flush(FRHICommandListImmediate& RHICmdList, FRHITransientAllocationStats* OutAllocationStats) override;

	uint32 GetPageSize() const { return PageSize; }

	uint32 GetPagePoolCount() const { return PagePools.Num(); }

	FRHITransientPagePoolCache& PagePoolCache;

private:
	static constexpr uint32 ResourceCacheGarbageCollectionLatency = 2;
	static constexpr uint64 KB = 1024;
	static constexpr uint64 MB = 1024 * KB;

	RHICORE_API void AllocateMemoryInternal(FRHITransientResource* Resource, const TCHAR* DebugName, const FRHITransientAllocationFences& Fences, bool bFastPoolRequested, float FastPoolPercentageRequested);
	RHICORE_API void DeallocateMemoryInternal(FRHITransientResource* Resource, const FRHITransientAllocationFences& Fences);

	//////////////////////////////////////////////////////////////////////////
	//! Platform API

	virtual FRHITransientTexture* CreateTextureInternal(
		const FRHITextureCreateInfo& CreateInfo,
		const TCHAR* DebugName,
		uint64 Hash) = 0;

	virtual FRHITransientBuffer* CreateBufferInternal(
		const FRHIBufferCreateInfo& CreateInfo,
		const TCHAR* DebugName,
		uint64 Hash) = 0;

	virtual void ReleaseTextureInternal(FRHITransientTexture* Texture) = 0;
	virtual void ReleaseBufferInternal(FRHITransientBuffer* Buffer) = 0;

	//////////////////////////////////////////////////////////////////////////

	template <typename FunctionType>
	void EnumeratePageSpans(const FRHITransientResource* Resource, FunctionType Function) const
	{
		const FRHITransientPageAllocation& PageAllocation = Resource->GetPageAllocation();

		for (const FRHITransientPagePoolAllocation& PoolAllocation : PageAllocation.PoolAllocations)
		{
			for (uint32 Index = PoolAllocation.SpanOffsetMin; Index < PoolAllocation.SpanOffsetMax; ++Index)
			{
				Function(PoolAllocation.Pool, PageAllocation.Spans[Index]);
			}
		}
	}

	FRHITransientMemoryStats Stats;
	TRHITransientResourceCache<FRHITransientTexture> Textures;
	TRHITransientResourceCache<FRHITransientBuffer> Buffers;

	TArray<FRHITransientPagePool*> PagePools;
	FRHITransientPagePool* FastPagePool = nullptr;
	uint64 CurrentCycle = 0;
	uint32 FirstNormalPagePoolIndex = 0;
	uint32 DeallocationCount = 0;
	uint32 PageSize = 0;
	uint32 PageMapCount = 0;
	uint32 PageAllocateCount = 0;
	uint32 PageSpanCount = 0;

	IF_RHICORE_TRANSIENT_ALLOCATOR_DEBUG(TSet<FRHITransientResource*> ActiveResources);
};
