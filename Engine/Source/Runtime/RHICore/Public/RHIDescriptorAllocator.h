// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "RHIDefinitions.h"
#include "Stats/Stats.h"

struct FRHIDescriptorAllocatorRange
{
	FRHIDescriptorAllocatorRange(uint32 InFirst, uint32 InLast) : First(InFirst), Last(InLast) {}
	uint32 First;
	uint32 Last;
};

struct FRHIDescriptorAllocation
{
	FRHIDescriptorAllocation(uint32 InStartIndex, uint32 InCount)
		: StartIndex(InStartIndex)
		, Count(InCount)
	{
	}

	uint32 StartIndex = 0;
	uint32 Count = 0;
};

class FRHIDescriptorAllocator
{
public:
	RHICORE_API FRHIDescriptorAllocator();
	RHICORE_API FRHIDescriptorAllocator(uint32 InNumDescriptors, TConstArrayView<TStatId> InStats);
	RHICORE_API ~FRHIDescriptorAllocator();

	RHICORE_API void Init(uint32 InNumDescriptors, TConstArrayView<TStatId> InStats);
	RHICORE_API void Shutdown();

	RHICORE_API TOptional<FRHIDescriptorAllocation> ResizeGrowAndAllocate(uint32 NewCapacity, uint32 NumAllocations);

	RHICORE_API TOptional<FRHIDescriptorAllocation> Allocate(uint32 NumDescriptors);
	RHICORE_API void Free(FRHIDescriptorAllocation Allocation);

	RHICORE_API FRHIDescriptorHandle Allocate(ERHIDescriptorType InType);
	RHICORE_API void Free(TConstArrayView<FRHIDescriptorHandle> InHandles);
	RHICORE_API void Free(FRHIDescriptorHandle InHandle);

	// Get the range of allocated descriptors. Useful for determining the smallest range to copy between heaps.
	RHICORE_API bool GetAllocatedRange(FRHIDescriptorAllocatorRange& OutRange);

	uint32 GetCapacity() const { return Capacity; }

private:

	RHICORE_API TOptional<FRHIDescriptorAllocation> AllocateInternal(uint32 NumDescriptors);
	RHICORE_API void FreeInternal(FRHIDescriptorAllocation Allocation);

	void RecordAlloc(uint32 Count)
	{
#if STATS
		for (TStatId Stat : Stats)
		{
			INC_DWORD_STAT_BY_FName(Stat.GetName(), Count);
		}
#endif
	}

	void RecordFree(uint32 Count)
	{
#if STATS
		for (TStatId Stat : Stats)
		{
			DEC_DWORD_STAT_BY_FName(Stat.GetName(), Count);
		}
#endif
	}

	TArray<FRHIDescriptorAllocatorRange> Ranges;
	uint32 Capacity = 0;

	FCriticalSection CriticalSection;

#if DO_CHECK
	// Debug tracking of outstanding allocations to detect double-free.
	struct FDebugDescriptorAllocation
	{
		uint32 Count = 0;
		TArray<uint64, TInlineAllocator<16>> Callstack;
	};
	TMap<uint32, FDebugDescriptorAllocation> DebugOutstandingAllocations;
#endif

#if STATS
	TArray<TStatId> Stats;
#endif
};


class FRHIHeapDescriptorAllocator : protected FRHIDescriptorAllocator
{
public:
	FRHIHeapDescriptorAllocator() = delete;
	RHICORE_API FRHIHeapDescriptorAllocator(ERHIDescriptorTypeMask InTypeMask, uint32 InDescriptorCount, TConstArrayView<TStatId> InStats);

	RHICORE_API FRHIDescriptorHandle Allocate(ERHIDescriptorType InType);
	RHICORE_API void Free(TConstArrayView<FRHIDescriptorHandle> InHandles);
	RHICORE_API void Free(FRHIDescriptorHandle InHandle);

	RHICORE_API TOptional<FRHIDescriptorAllocation> Allocate(uint32 NumDescriptors);
	RHICORE_API void Free(FRHIDescriptorAllocation Allocation);

	using FRHIDescriptorAllocator::GetAllocatedRange;
	using FRHIDescriptorAllocator::GetCapacity;
	using FRHIDescriptorAllocator::ResizeGrowAndAllocate;

	ERHIDescriptorTypeMask GetTypeMask() const
	{
		return TypeMask;
	}

	bool HandlesAllocation(ERHIDescriptorType InType) const
	{
		return EnumHasAnyFlags(GetTypeMask(), RHIDescriptorTypeMaskFromType(InType));
	}

	bool HandlesAllocations(ERHIDescriptorTypeMask InTypeMask) const
	{
		return EnumHasAllFlags(GetTypeMask(), InTypeMask);
	}

private:
	const ERHIDescriptorTypeMask TypeMask;
};

class FRHIOffsetHeapDescriptorAllocator : protected FRHIHeapDescriptorAllocator
{
public:
	FRHIOffsetHeapDescriptorAllocator() = delete;
	RHICORE_API FRHIOffsetHeapDescriptorAllocator(ERHIDescriptorTypeMask InTypeMask, uint32 InDescriptorCount, uint32 InHeapOffset, TConstArrayView<TStatId> InStats);

	RHICORE_API FRHIDescriptorHandle Allocate(ERHIDescriptorType InType);
	RHICORE_API void Free(FRHIDescriptorHandle InHandle);

	RHICORE_API TOptional<FRHIDescriptorAllocation> Allocate(uint32 NumDescriptors);
	RHICORE_API void Free(FRHIDescriptorAllocation Allocation);

	using FRHIHeapDescriptorAllocator::GetCapacity;
	using FRHIHeapDescriptorAllocator::GetTypeMask;
	using FRHIHeapDescriptorAllocator::HandlesAllocation;
	using FRHIHeapDescriptorAllocator::HandlesAllocations;

private:
	// Offset from start of heap we belong to
	const uint32 HeapOffset;
};
