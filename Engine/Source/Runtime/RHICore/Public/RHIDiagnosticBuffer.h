// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIBreadcrumbs.h"

/// Common base for platform-specific implementations of GPU diagnostic buffer.
/// The diagnostic buffer contains GPU messages (debug logs, shader asserts, etc.) and GPU progress breadcrumbs.
/// This buffer is persistently mapped and can be accessed on CPU at any point, including after a GPU crash has been detected.
/// Platform-specific code is responsible for allocating the actual underlying resource and binding it to shaders that need it.
/// Diagnostic buffer functionality may be used independently of GPU breadcrumbs.
class FRHIDiagnosticBuffer
{
public:

	// Counterpart to UEDiagnosticBuffer in shader code
	struct FLane
	{
		uint32 Counter;
		uint32 MessageID;
		union
		{
			int32  AsInt[4];
			uint32 AsUint[4];
			float  AsFloat[4];
		} Payload;
	};

	static_assert(sizeof(FLane) == 6 * sizeof(uint32), "Remember to change UEDiagnosticBuffer layout in the shaders when changing FLane");

	struct FQueue
	{
		// Counterpart to UEDiagnosticMaxLanes in shader code
		static constexpr uint32 MaxLanes = 64;
		FLane Lanes[MaxLanes];

		#if WITH_RHI_BREADCRUMBS
		// GPU breadcrumb markers
		uint32 MarkerIn;
		uint32 MarkerOut;
		#endif
	}; 
	
	static constexpr uint32 SizeInBytes = sizeof(FQueue);

	// Persistently mapped diagnostic buffer data, initialized by platform-specific code
	FQueue* Data = nullptr;

	// Log the GPU progress of the given queue to the Error log if breadcrumb data is available
	FString RHICORE_API GetShaderDiagnosticMessages(uint32 DeviceIndex, uint32 QueueIndex, const TCHAR* QueueName);
};
