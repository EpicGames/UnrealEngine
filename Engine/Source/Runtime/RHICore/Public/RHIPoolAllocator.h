// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "HAL/CriticalSection.h"
#include "RHIFwd.h"
#include "RHIPipeline.h"

class FRHICommandListBase;
struct FScriptContainerElement;

// Pre declares
class FRHIPoolAllocator;

/**
@brief Owner of the pool allocation data which needs to handle memory move operations
**/
class FRHIPoolResource
{
};


/**
@brief Resource type supported by the pools (can be any or all)
**/
enum class ERHIPoolResourceTypes
{
	Buffers			= 0x1,
	RTDSTextures	= 0x2,
	NonRTDSTextures = 0x4,
	All				= Buffers | RTDSTextures | NonRTDSTextures,
};

/**
@brief Type of allocation strategy
**/
enum class ERHIPoolAllocationStrategy
{
	Immediate,
	Async,
	Count
};

/**
@brief Status of the Async pool allocation
**/
enum class ERHIPoolAllocationStatus
{
	Idle,
	Allocating,
	ShutDown,
	Count
};

/**
@brief Pool allocator internal data
**/
struct FRHIPoolAllocationData
{
public:

	// Reset to unknown state
	RHICORE_API void Reset();

	// Initialize options
	RHICORE_API void InitAsHead(int16 InPoolIndex);
	RHICORE_API void InitAsFree(int16 InPoolIndex, uint32 InSize, uint32 InAlignment, uint32 InOffset);
	RHICORE_API void InitAsAllocated(uint32 InSize, uint32 InPoolAlignment, uint32 InAllocationAlignment, FRHIPoolAllocationData* InFree);
	RHICORE_API void MoveFrom(FRHIPoolAllocationData& InAllocated, bool InLocked);

	// Free block operation (TODO: make internal)
	RHICORE_API void MarkFree(uint32 InPoolAlignment, uint32 InAllocationAlignment);

	// Alias operations (NOTE: currently not accessed when pool lock is taken, needs to be fixed)
	RHICORE_API void AddAlias(FRHIPoolAllocationData* InOther);
	RHICORE_API void RemoveAlias();
	FRHIPoolAllocationData* GetFirstAlias() const { return AliasAllocation; }

	// Type getters
	bool IsHead() const { return GetAllocationType() == EAllocationType::Head; }
	bool IsFree() const { return GetAllocationType() == EAllocationType::Free; }
	bool IsAllocated() const { return GetAllocationType() == EAllocationType::Allocated; }

	// Allocation data getters
	uint32 GetSize() const { return Size; }
	uint32 GetAlignment() const { return Alignment; }
	uint32 GetOffset() const { return Offset; }
	int16 GetPoolIndex() const { return PoolIndex; }

	// Linked list getters
	FRHIPoolAllocationData* GetNext() const { return NextAllocation; }
	FRHIPoolAllocationData* GetPrev() const { return PreviousAllocation; }

	// Owner getter/setter - opaque data
	void SetOwner(FRHIPoolResource* InOwner) { check(Owner == nullptr); Owner = InOwner; }
	FRHIPoolResource* GetOwner() const { return Owner; }

	// Unlock can be public - no lock required
	bool IsLocked() const { return (Locked == 1); }
	void Unlock() { check(Locked == 1); Locked = 0; }

private:

	// Unlock data block again - actual lock on owned allocator should be taken then
	friend class FRHIMemoryPool;
	friend class FRHIPoolAllocator;
	void Lock() { check(Locked == 0); Locked = 1; }

	RHICORE_API void Merge(FRHIPoolAllocationData* InOther);

	// Linked list operation
	RHICORE_API void RemoveFromLinkedList();
	RHICORE_API void AddBefore(FRHIPoolAllocationData* InOther);
	RHICORE_API void AddAfter(FRHIPoolAllocationData* InOther);
		
	enum class EAllocationType : uint8
	{
		Unknown,
		Free,
		Allocated,
		Head,
	};
	EAllocationType GetAllocationType() const { return (EAllocationType) Type; }
	void SetAllocationType(EAllocationType InType) { Type = (uint32)InType; }

	uint32 Size;
	uint32 Alignment;
	uint32 Offset;
	uint32 PoolIndex : 16;
	uint32 Type : 4;
	// Allocation data is locked after creation until initial data is filled in to make sure it's not being
	// defragmented then. If a block is locked for read/write from CPU side then it also needs be locked.
	// While it's defragmented then the block is also locked so make sure to wait for the GPU before destroying it
	uint32 Locked : 1;		
	uint32 Unused : 11;

	// TODO: Link list data could be stored as index into a fixed array managed by the memory pool. Then only a uin16 or uint32 is needed here to get the next and previous index
	FRHIPoolAllocationData* PreviousAllocation;
	FRHIPoolAllocationData* NextAllocation;
	FRHIPoolAllocationData* AliasAllocation;		// Linked list of aliases

	FRHIPoolResource* Owner;
};


//////////////////////////////////////////////////////////////////////////
// FRHIMemoryPool
//////////////////////////////////////////////////////////////////////////

/**
@brief Pool which stores for each allocation the previous and next allocation.
	   Each block is either free or allocated.
**/
class FRHIMemoryPool
{
public:

	static constexpr int16 InvalidPoolIndex = -1;

	enum class EFreeListOrder
	{
		SortBySize,
		SortByOffset,
	};

	// Constructor
	RHICORE_API FRHIMemoryPool(int16 InPoolIndex, uint64 InPoolSize, uint32 InPoolAlignment, ERHIPoolResourceTypes InSupportedResourceTypes, EFreeListOrder InFreeListOrder);
	RHICORE_API virtual ~FRHIMemoryPool();

	// Setup/Shutdown
	RHICORE_API virtual void Init();
	RHICORE_API virtual void Destroy();
	RHICORE_API void InitFreeBlocks();

	// AllocationData functions: allocate, free, move, ...
	RHICORE_API bool TryAllocate(uint32 InSizeInBytes, uint32 InAllocationAlignment, ERHIPoolResourceTypes InAllocationResourceType, FRHIPoolAllocationData& AllocationData);
	RHICORE_API void Deallocate(FRHIPoolAllocationData& AllocationData);

	// Bookkeeping and clearing
	RHICORE_API void TryClear(FRHIContextArray const& Contexts, FRHIPoolAllocator* InAllocator, uint32 InMaxCopySize, uint32& CopySize, const TArray<FRHIMemoryPool*>& InTargetPools);

	// Getters
	int16 GetPoolIndex() const { return PoolIndex; }
	uint64 GetPoolSize() const { return PoolSize; }
	uint64 GetFreeSize() const { return FreeSize; }
	uint64 GetUsedSize() const { return (PoolSize - FreeSize); }
	uint32 GetPoolAlignment() const { return PoolAlignment; }
	uint32 GetAlignmentWaste() const { return AligmnentWaste; }
	uint32 GetAllocatedBlocks() const { return AllocatedBlocks; }
	ERHIPoolResourceTypes GetSupportedResourceTypes() const { return SupportedResourceTypes; }
	bool IsResourceTypeSupported(ERHIPoolResourceTypes InType) const { return EnumHasAnyFlags(SupportedResourceTypes, InType); }
	bool IsEmpty() const { return GetUsedSize() == 0; }
	bool IsFull() const { return FreeSize == 0; }

	// Index setter for async allocation
	void SetPoolIndex(int16 InPoolIndex) { PoolIndex = InPoolIndex; }

	// Static public helper functions
	static RHICORE_API uint32 GetAlignedSize(uint32 InSizeInBytes, uint32 InPoolAlignment, uint32 InAllocationAlignment);
	static RHICORE_API uint32 GetAlignedOffset(uint32 InOffset, uint32 InPoolAlignment, uint32 InAllocationAlignment);

protected:
		
	// Free block management helper function
	RHICORE_API int32 FindFreeBlock(uint32 InSizeInBytes, uint32 InAllocationAlignment) const;
	RHICORE_API FRHIPoolAllocationData* AddToFreeBlocks(FRHIPoolAllocationData* InFreeBlock);
	RHICORE_API void RemoveFromFreeBlocks(FRHIPoolAllocationData* InFreeBlock);

	// Get/Release allocation data blocks
	RHICORE_API FRHIPoolAllocationData* GetNewAllocationData();
	RHICORE_API void ReleaseAllocationData(FRHIPoolAllocationData* InData);

	// Validate the linked list data - slow
	RHICORE_API void Validate();

	// Const creation members
	int16 PoolIndex;
	const uint64 PoolSize;
	const uint32 PoolAlignment;
	const ERHIPoolResourceTypes SupportedResourceTypes;
	const EFreeListOrder FreeListOrder;

	// Stats
	uint64 FreeSize;
	uint64 AligmnentWaste;
	uint32 AllocatedBlocks;

	// Special start block to mark the beginning and the end of the linked list
	FRHIPoolAllocationData HeadBlock;

	// Actual Free blocks
	TArray<FRHIPoolAllocationData*> FreeBlocks;

	// Allocation data pool used to reduce allocation overhead
	TArray<FRHIPoolAllocationData*> AllocationDataPool;
};


//////////////////////////////////////////////////////////////////////////
// FRHIPoolAllocator
//////////////////////////////////////////////////////////////////////////

/**
@brief Manages an array of FRHIMemoryPool and supports defragmentation between multiple pools
**/
class FRHIPoolAllocator
{
public:

	// Constructor
	RHICORE_API FRHIPoolAllocator(uint64 InDefaultPoolSize, uint32 InPoolAlignment, uint32 InMaxAllocationSize, FRHIMemoryPool::EFreeListOrder InFreeListOrder, bool InDefragEnabled, ERHIPoolAllocationStrategy InPoolAllocationStrategy, uint16 InOverFlowPoolsCount);
	RHICORE_API virtual ~FRHIPoolAllocator();

	// Setup/Shutdown
	RHICORE_API void Initialize(ERHIPoolResourceTypes InAllocationResourceType);
	RHICORE_API void Destroy();

	// Defrag & cleanup operation
	RHICORE_API void Defrag(FRHIContextArray const& Contexts, uint32 InMaxCopySize, uint32& CurrentCopySize);

	// Stats
	RHICORE_API void UpdateMemoryStats(uint32& IOMemoryAllocated, uint32& IOMemoryUsed, uint32& IOMemoryFree, uint32& IOMemoryEndFree, uint32& IOAlignmentWaste, uint32& IOAllocatedPageCount, uint32& IOFullPageCount);

protected:
		
	// Supported allocator operations
	RHICORE_API bool TryAllocateInternal(uint32 InSizeInBytes, uint32 InAllocationAlignment, ERHIPoolResourceTypes InAllocationResourceType, FRHIPoolAllocationData& AllocationData);
	RHICORE_API void DeallocateInternal(FRHIPoolAllocationData& AllocationData);

	RHICORE_API void SortPools();

	RHICORE_API void PoolAsyncAllocate(uint32 InSizeInBytes, uint32 InAllocationAlignment, ERHIPoolResourceTypes InAllocationResourceType);
	
	// Helper function to create a new platform specific pool
	virtual FRHIMemoryPool* CreateNewPool(int16 InPoolIndex, uint32 InMinimumAllocationSize, ERHIPoolResourceTypes InAllocationResourceType) = 0;

	// Handle a rhi specific defrag op
	friend class FRHIMemoryPool;
	virtual bool HandleDefragRequest(FRHIContextArray const& Contexts, FRHIPoolAllocationData* InSourceBlock, FRHIPoolAllocationData& InTmpTargetBlock) = 0;

	// Const creation members - used to create new pools
	const uint64 DefaultPoolSize;
	const uint32 PoolAlignment;
	const uint64 MaxAllocationSize;
	const FRHIMemoryPool::EFreeListOrder FreeListOrder;
	const bool bDefragEnabled;

	// Critical section to lock access to the pools
	FCriticalSection CS;

	// Actual pools managing each a linked list of allocations
	TArray<FRHIMemoryPool*> Pools;

	// Last defrag pool index - defrag is time slices to reduce CPU overhead
	int32 LastDefragPoolIndex;

	// Allocation order of the pools when performing new allocations
	// Can' sort the pools directly because pool index is stored in the allocation info
	TArray<uint32> PoolAllocationOrder;

	// Stats
	uint32 TotalAllocatedBlocks;

	// Array of pre-allocated pools
	TArray<FRHIMemoryPool*> OverFlowPools;

	// Pool allocation strategy
	ERHIPoolAllocationStrategy	PoolAllocationStrategy = ERHIPoolAllocationStrategy::Immediate;

	// Pool allocation status
	ERHIPoolAllocationStatus	PoolAllocationStatus = ERHIPoolAllocationStatus::Idle;

	// Max pre-allocated pool
	int16 OverFlowPoolsCount	= 1;
};
