// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;

public class RHICore : ModuleRules
{
	public RHICore(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddAll("RHI", "RenderCore");
		PrivateDependencyModuleNames.Add("Core");

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			PublicDefinitions.Add("RHICORE_PLATFORM_DXGI_H=<dxgi1_4.h>");
		}

		AddEngineThirdPartyPrivateStaticDependencies(Target, "NVAftermath");
		AddEngineThirdPartyPrivateStaticDependencies(Target, "IntelExtensionsFramework");
	}
}
